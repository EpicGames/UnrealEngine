// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandaloneRendererPrivate.h"

#include "OpenGL/SlateOpenGLRenderer.h"

FSlateOpenGLContext::FSlateOpenGLContext()
:	WindowHandle(NULL)
,	Context(NULL)
{
}

FSlateOpenGLContext::~FSlateOpenGLContext()
{
	Destroy();
}

void FSlateOpenGLContext::Initialize( void* InWindow, const FSlateOpenGLContext* SharedContext )
{
}

void FSlateOpenGLContext::Destroy()
{
}

void FSlateOpenGLContext::MakeCurrent()
{
}
