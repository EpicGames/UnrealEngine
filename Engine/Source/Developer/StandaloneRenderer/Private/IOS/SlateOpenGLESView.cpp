// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandaloneRendererPrivate.h"
#include "IOS/SlateOpenGLESView.h"
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSInputInterface.h"

@implementation SlateOpenGLESView

@synthesize Context;

- (id)initWithFrame:(CGRect)FrameRect
{
	if ((self = [super initWithFrame:FrameRect]))
	{
		self.opaque = NO;
		self.backgroundColor = [UIColor redColor];
	}
	return self;
}

/**
 * Pass touch events to the input queue for slate to pull off of, and trigger the debug console.
 *
 * @param View The view the event happened in
 * @param Touches Set of touch events from the OS
 */
void HandleSlateAppTouches(UIView* View, NSSet* Touches, TouchType Type)
{
	// ignore touches until game is booted
	SlateOpenGLESView* GLView = static_cast<SlateOpenGLESView*>( View );
	CGFloat Scale = [[UIScreen mainScreen] scale];

	TArray<TouchInput> TouchesArray;
	for (UITouch* Touch in Touches)
	{
		// get info from the touch
		CGPoint Loc = [Touch locationInView:View];
		CGPoint PrevLoc = [Touch previousLocationInView:View];

		//@todo zombie - Get the actual Controller ID, and touch pad index. Also, figure out how to deal with touch phases.
		TouchInput TouchMessage;
		TouchMessage.Handle = (int)Touch;
		TouchMessage.Type = Type;
		TouchMessage.Position = FVector2D(Loc.x, Loc.y) * Scale;
		TouchMessage.LastPosition = FVector2D(PrevLoc.x, PrevLoc.y) * Scale;
		TouchesArray.Add(TouchMessage);
	}

	FIOSInputInterface::QueueTouchInput(TouchesArray);
}

/**
 * Handle the various touch types from the OS
 *
 * @param touches Array of touch events
 * @param event Event information
 */
- (void) touchesBegan:(NSSet*)touches withEvent:(UIEvent*)event 
{
	HandleSlateAppTouches(self, touches, TouchBegan);
}

- (void) touchesMoved:(NSSet*)touches withEvent:(UIEvent*)event
{
	HandleSlateAppTouches(self, touches, TouchMoved);
}

- (void) touchesEnded:(NSSet*)touches withEvent:(UIEvent*)event
{
	HandleSlateAppTouches(self, touches, TouchEnded);
}

- (void) touchesCancelled:(NSSet*)touches withEvent:(UIEvent*)event
{
	HandleSlateAppTouches(self, touches, TouchEnded);
}

@end

@implementation SlateOpenGLESViewController

/**
 * The ViewController was created, so now we need to create our view to be controlled (an EAGLView)
 */
- (void) loadView
{
	// get the landcape size of the screen
	CGRect Frame = [[UIScreen mainScreen] bounds];
	EAGLContext* Context = [[EAGLContext alloc] initWithAPI: kEAGLRenderingAPIOpenGLES2];
	SlateOpenGLESView* View = [[SlateOpenGLESView alloc] initWithFrame:Frame context:Context];
	self.view = View;

	// pass ownership to the view
	View.Context = Context;
	[Context release];
	
	// settings copied from InterfaceBuilder
	self.wantsFullScreenLayout = YES;
	self.view.clearsContextBeforeDrawing = NO;
	self.view.multipleTouchEnabled = NO;
	((GLKView*)self.view).enableSetNeedsDisplay = YES;
}

/**
 * Tell the OS that our view controller can auto-rotate between the two landscape modes
 */
- (BOOL)shouldAutorotateToInterfaceOrientation:(UIInterfaceOrientation)interfaceOrientation
{
	return YES;
}

@end
