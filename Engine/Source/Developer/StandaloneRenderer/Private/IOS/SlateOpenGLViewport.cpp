// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandaloneRendererPrivate.h"
#include "OpenGL/SlateOpenGLRenderer.h"
#include "IOS/SlateOpenGLESView.h"

#import <OpenGLES/EAGLDrawable.h>
#import <QuartzCore/QuartzCore.h>


uint GDeviceWidth = 0;
uint GDeviceHeight = 0;

FSlateOpenGLViewport::FSlateOpenGLViewport()
:	bFullscreen( true )
{
}

void FSlateOpenGLViewport::Initialize( TSharedRef<SWindow> InWindow, const FSlateOpenGLContext& SharedContext )
{
	TSharedRef<FGenericWindow> NativeWindow = InWindow->GetNativeWindow().ToSharedRef();
	RenderingContext.WindowHandle = (UIWindow*)NativeWindow->GetOSWindowHandle();

	FSlateRect FullScreenSize = InWindow->GetFullScreenInfo();

	const int Width = FMath::TruncToInt(FullScreenSize.Right - FullScreenSize.Left);
	const int Height = FMath::TruncToInt(FullScreenSize.Bottom - FullScreenSize.Top);

	// iOS should always be fullscreen and at the origin
	InWindow->SetCachedSize( FVector2D(Width, Height) );
	InWindow->SetCachedScreenPosition( FVector2D(0, 0));

	ProjectionMatrix = CreateProjectionMatrix( Width, Height );

	ViewportRect.Right = Width;
	ViewportRect.Bottom = Height;
	ViewportRect.Top = 0;
	ViewportRect.Left = 0;

	UIWindow* MainWindow = (UIWindow*)RenderingContext.WindowHandle;
	SlateOpenGLESViewController* RootView = (SlateOpenGLESViewController *)MainWindow.rootViewController;
	SlateOpenGLESView * View = (SlateOpenGLESView *)[RootView view];
	RenderingContext.Context = View.context;
	[EAGLContext setCurrentContext: RenderingContext.Context];
}

void FSlateOpenGLViewport::Destroy()
{
	if( RenderingContext.WindowHandle )
	{
		UIWindow* MainWindow = (UIWindow*)RenderingContext.WindowHandle;
		SlateOpenGLESView * View = (SlateOpenGLESView *)[MainWindow.rootViewController view];
		[View removeFromSuperview];
		RenderingContext.WindowHandle = NULL;
		RenderingContext.Context = NULL; // This is released in content view's destructor
	}
}

void FSlateOpenGLViewport::MakeCurrent()
{
	UIWindow* MainWindow = (UIWindow*)RenderingContext.WindowHandle;
	SlateOpenGLESView * View = (SlateOpenGLESView *)[MainWindow.rootViewController view];
	[View bindDrawable];
}

void FSlateOpenGLViewport::SwapBuffers()
{
	UIWindow* MainWindow = (UIWindow*)RenderingContext.WindowHandle;
	SlateOpenGLESView * View = (SlateOpenGLESView *)[MainWindow.rootViewController view];
	[View display];
}

void FSlateOpenGLViewport::Resize( int Width, int Height, bool bInFullscreen )
{
	UIWindow* MainWindow = (UIWindow*)RenderingContext.WindowHandle;
	SlateOpenGLESView * View = (SlateOpenGLESView *)[MainWindow.rootViewController view];

	int ViewportWidth = Width;
	int ViewportHeight = Height;

	if(bInFullscreen)
	{
		CGRect Frame = [[UIScreen mainScreen] bounds];
		CGFloat Scale = [[UIScreen mainScreen] scale];

		IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
		if (!AppDelegate.bDeviceInPortraitMode)
		{
			Swap<float>(Frame.size.width, Frame.size.height);
		}

		ViewportWidth = FMath::TruncToInt(Frame.size.width * Scale);
		ViewportHeight = FMath::TruncToInt(Frame.size.height * Scale);
	}

	View.frame = CGRectMake(0, 0, ViewportWidth, ViewportHeight);

	ProjectionMatrix = CreateProjectionMatrix( ViewportWidth, ViewportHeight );

	ViewportRect.Right = ViewportWidth;
	ViewportRect.Bottom = ViewportHeight;
	ViewportRect.Top = 0;
	ViewportRect.Left = 0;
}
