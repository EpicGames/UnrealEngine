// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/CommandLine.h"
#include "OpenGL/SlateOpenGLExtensions.h"
#include "StandaloneRendererLog.h"
#include "OpenGL/SlateOpenGLRenderer.h"
#include "HAL/PlatformApplicationMisc.h"

#if defined(GL_ARB_debug_output) || defined(GL_KHR_debug)

/**
 * Map GL_DEBUG_SOURCE_*_ARB to a human-readable string.
 */
static const TCHAR* GetOpenGLDebugSourceStringARB(GLenum Source)
{
	static const TCHAR* SourceStrings[] =
	{
		TEXT("API"),
		TEXT("System"),
		TEXT("ShaderCompiler"),
		TEXT("ThirdParty"),
		TEXT("Application"),
		TEXT("Other")
	};

	if (Source >= GL_DEBUG_SOURCE_API_ARB && Source <= GL_DEBUG_SOURCE_OTHER_ARB)
	{
		return SourceStrings[Source - GL_DEBUG_SOURCE_API_ARB];
	}
	return TEXT("Unknown");
}

/**
 * Map GL_DEBUG_TYPE_*_ARB to a human-readable string.
 */
static const TCHAR* GetOpenGLDebugTypeStringARB(GLenum Type)
{
	static const TCHAR* TypeStrings[] =
	{
		TEXT("Error"),
		TEXT("Deprecated"),
		TEXT("UndefinedBehavior"),
		TEXT("Portability"),
		TEXT("Performance"),
		TEXT("Other")
	};

	if (Type >= GL_DEBUG_TYPE_ERROR_ARB && Type <= GL_DEBUG_TYPE_OTHER_ARB)
	{
		return TypeStrings[Type - GL_DEBUG_TYPE_ERROR_ARB];
	}
#ifdef GL_KHR_debug
	{
		static const TCHAR* KHRTypeStrings[] =
		{
			TEXT("Marker"),
			TEXT("PushGroup"),
			TEXT("PopGroup"),
		};

		if (Type >= GL_DEBUG_TYPE_MARKER && Type <= GL_DEBUG_TYPE_POP_GROUP)
		{
			return KHRTypeStrings[Type - GL_DEBUG_TYPE_MARKER];
		}
	}
#endif
	return TEXT("Unknown");
}

/**
 * Map GL_DEBUG_SEVERITY_*_ARB to a human-readable string.
 */
static const TCHAR* GetOpenGLDebugSeverityStringARB(GLenum Severity)
{
	static const TCHAR* SeverityStrings[] =
	{
		TEXT("High"),
		TEXT("Medium"),
		TEXT("Low")
	};

	if (Severity >= GL_DEBUG_SEVERITY_HIGH_ARB && Severity <= GL_DEBUG_SEVERITY_LOW_ARB)
	{
		return SeverityStrings[Severity - GL_DEBUG_SEVERITY_HIGH_ARB];
	}
#ifdef GL_KHR_debug
	if(Severity == GL_DEBUG_SEVERITY_NOTIFICATION)
		return TEXT("Notification");
#endif
	return TEXT("Unknown");
}

/**
 * OpenGL debug message callback. Conforms to GLDEBUGPROCARB.
 */
static void APIENTRY OpenGLDebugMessageCallbackARB(
	GLenum Source,
	GLenum Type,
	GLuint Id,
	GLenum Severity,
	GLsizei Length,
	const GLchar* Message,
	GLvoid* UserParam)
{
#if !NO_LOGGING
	const TCHAR* SourceStr = GetOpenGLDebugSourceStringARB(Source);
	const TCHAR* TypeStr = GetOpenGLDebugTypeStringARB(Type);
	const TCHAR* SeverityStr = GetOpenGLDebugSeverityStringARB(Severity);

	ELogVerbosity::Type Verbosity = ELogVerbosity::Warning;
	if (Type == GL_DEBUG_TYPE_ERROR_ARB && Severity == GL_DEBUG_SEVERITY_HIGH_ARB)
	{
		Verbosity = ELogVerbosity::Fatal;
	}

	if ((Verbosity & ELogVerbosity::VerbosityMask) <= FLogCategoryLogStandaloneRenderer::CompileTimeVerbosity)
	{
		if (!LogStandaloneRenderer.IsSuppressed(Verbosity))
		{
			FMsg::Logf(__FILE__, __LINE__, LogStandaloneRenderer.GetCategoryName(), Verbosity,
				TEXT("[%s][%s][%s][%u] %s"),
				SourceStr,
				TypeStr,
				SeverityStr,
				Id,
				ANSI_TO_TCHAR(Message)
				);
		}

		// this is a debugging code to catch VIDEO->HOST copying
		if (Id == 131186)
		{
			int A = 5;
		}
	}
#endif
}

#endif // defined(GL_ARB_debug_output) || defined(GL_KHR_debug)

/**
 * Enable/Disable debug context from the commandline
 */
static bool PlatformOpenGLDebugCtx()
{
#if UE_BUILD_DEBUG
	return ! FParse::Param(FCommandLine::Get(),TEXT("openglNoDebug"));
#else
	return FParse::Param(FCommandLine::Get(),TEXT("openglDebug"));;
#endif
}

static SDL_Window* CreateDummyGLWindow()
{
	FPlatformApplicationMisc::InitSDL(); //	will not initialize more than once

#if DO_CHECK
	uint32 InitializedSubsystems = SDL_WasInit(SDL_INIT_VIDEO);
	check(InitializedSubsystems & SDL_INIT_VIDEO);
#endif // DO_CHECK

	// Create a dummy window.
	SDL_Window *h_wnd = SDL_CreateWindow(	NULL,
		1, 1,
		SDL_WINDOW_OPENGL | SDL_WINDOW_BORDERLESS | SDL_WINDOW_HIDDEN );

	return h_wnd;
}

FSlateOpenGLContext::FSlateOpenGLContext()
	: WindowHandle(NULL)
	, Context(NULL)
	, bReleaseWindowOnDestroy(false)
#if LINUX_USE_OPENGL_3_2
	, VertexArrayObject(0)
#endif // LINUX_USE_OPENGL_3_2
{
}

FSlateOpenGLContext::~FSlateOpenGLContext()
{
	Destroy();
}

void FSlateOpenGLContext::Initialize( void* InWindow, const FSlateOpenGLContext* SharedContext )
{
	WindowHandle = (SDL_Window*)InWindow;

	if	( WindowHandle == NULL )
	{
		WindowHandle = CreateDummyGLWindow();
		bReleaseWindowOnDestroy = true;
	}

	if (SDL_GL_LoadLibrary(nullptr) == false)
	{
		FString SdlError(ANSI_TO_TCHAR(SDL_GetError()));
		UE_LOGF(LogInit, Fatal, "FSlateOpenGLContext::Initialize - Unable to dynamically load libGL: %ls\n.", *SdlError);
	}

	int OpenGLMajorVersionToUse = 2;
	int OpenGLMinorVersionToUse = 1;

#if LINUX_USE_OPENGL_3_2
	OpenGLMajorVersionToUse = 3;
	OpenGLMinorVersionToUse = 2;
#endif // LINUX_USE_OPENGL_3_2

	SDL_GL_SetAttribute( SDL_GL_CONTEXT_MAJOR_VERSION, OpenGLMajorVersionToUse );
	SDL_GL_SetAttribute( SDL_GL_CONTEXT_MINOR_VERSION, OpenGLMinorVersionToUse );
	SDL_GL_SetAttribute( SDL_GL_CONTEXT_PROFILE_MASK, SDL_GL_CONTEXT_PROFILE_CORE );

	if (PlatformOpenGLDebugCtx())
	{
		SDL_GL_SetAttribute(SDL_GL_CONTEXT_FLAGS, SDL_GL_CONTEXT_DEBUG_FLAG);
	}

	if( SharedContext )
	{
		SDL_GL_SetAttribute( SDL_GL_SHARE_WITH_CURRENT_CONTEXT, 1 );
		SDL_GL_MakeCurrent( SharedContext->WindowHandle, SharedContext->Context );
	}
	else
	{
		SDL_GL_SetAttribute( SDL_GL_SHARE_WITH_CURRENT_CONTEXT, 0 );
	}

	UE_LOGF(LogInit, Log, "FSlateOpenGLContext::Initialize - creating OpenGL %d.%d context", OpenGLMajorVersionToUse, OpenGLMinorVersionToUse);
	Context = SDL_GL_CreateContext( WindowHandle );
	if (Context == nullptr)
	{
		FString SdlError(ANSI_TO_TCHAR(SDL_GetError()));
		
		// ignore errors getting version, it will be clear from the logs
		int OpenGLMajorVersion = -1;
		int OpenGLMinorVersion = -1;
		SDL_GL_GetAttribute( SDL_GL_CONTEXT_MAJOR_VERSION, &OpenGLMajorVersion );
		SDL_GL_GetAttribute( SDL_GL_CONTEXT_MINOR_VERSION, &OpenGLMinorVersion );

		UE_LOGF(LogInit, Fatal, "FSlateOpenGLContext::Initialize - Could not create OpenGL %d.%d context, SDL error: '%ls'", 
				OpenGLMajorVersion, OpenGLMinorVersion,
				*SdlError
			);
		// unreachable
		return;
	}
	SDL_GL_MakeCurrent( WindowHandle, Context );

	LoadOpenGLExtensions();

	if (PlatformOpenGLDebugCtx())
	{
		if (glDebugMessageCallbackARB)
		{
		#if GL_ARB_debug_output || GL_KHR_debug
			// Synchronous output can slow things down, but we'll get better callstack if breaking in or crashing in the callback. This is debug only after all.
			glEnable(GL_DEBUG_OUTPUT_SYNCHRONOUS);
			glDebugMessageCallbackARB(GLDEBUGPROCARB(OpenGLDebugMessageCallbackARB), /*UserParam=*/ nullptr);
			checkf(glGetError() == GL_NO_ERROR, TEXT("Could not register glDebugMessageCallbackARB()"));

			if(glDebugMessageControlARB)
			{
				glDebugMessageControlARB(GL_DEBUG_SOURCE_APPLICATION_ARB, GL_DEBUG_TYPE_MARKER, GL_DONT_CARE, 0, NULL, GL_FALSE);
				glDebugMessageControlARB(GL_DEBUG_SOURCE_APPLICATION_ARB, GL_DEBUG_TYPE_PUSH_GROUP, GL_DONT_CARE, 0, NULL, GL_FALSE);
				glDebugMessageControlARB(GL_DEBUG_SOURCE_APPLICATION_ARB, GL_DEBUG_TYPE_POP_GROUP, GL_DONT_CARE, 0, NULL, GL_FALSE);
		#ifdef GL_KHR_debug
				glDebugMessageControlARB(GL_DEBUG_SOURCE_API_ARB, GL_DEBUG_TYPE_OTHER_ARB, GL_DEBUG_SEVERITY_NOTIFICATION, 0, NULL, GL_FALSE);
		#endif
				UE_LOGF(LogStandaloneRenderer,Verbose,"disabling reporting back of debug groups and markers to the OpenGL debug output callback");
			}
		#endif // GL_ARB_debug_output || GL_KHR_debug
		}
	}

#if LINUX_USE_OPENGL_3_2
	// one Vertex Array Object is reportedly needed for OpenGL 3.2+ core profiles
	glGenVertexArrays(1, &VertexArrayObject);
	glBindVertexArray(VertexArrayObject);
#endif // LINUX_USE_OPENGL_3_2
}

void FSlateOpenGLContext::Destroy()
{
	if	( WindowHandle != NULL )
	{
		SDL_GL_MakeCurrent( NULL, NULL );
#if LINUX_USE_OPENGL_3_2
		glDeleteVertexArrays(1, &VertexArrayObject);
#endif // LINUX_USE_OPENGL_3_2
		SDL_GL_DestroyContext( Context );

		if	( bReleaseWindowOnDestroy )
		{
			SDL_DestroyWindow( WindowHandle );
			// we will tear down SDL in PlatformTearDown()
		}
		WindowHandle = NULL;

	}
}

void FSlateOpenGLContext::MakeCurrent()
{
	if	( WindowHandle )
	{
		if(SDL_GL_MakeCurrent( WindowHandle, Context ) == 0)
		{
#if LINUX_USE_OPENGL_3_2
			glBindVertexArray(VertexArrayObject);
#endif // LINUX_USE_OPENGL_3_2
		}
	}
}
