// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "GenericPlatform/GenericWindow.h"
#include "Widgets/SWindow.h"

#include "OpenGL/SlateOpenGLRenderer.h"

FSlateOpenGLViewport::FSlateOpenGLViewport()
:	bFullscreen(false)
{
}

void FSlateOpenGLViewport::Initialize( TSharedRef<SWindow> InWindow, const FSlateOpenGLContext& SharedContext )
{
	TSharedRef<FGenericWindow> NativeWindow = InWindow->GetNativeWindow().ToSharedRef();
	RenderingContext.Initialize(NativeWindow->GetOSWindowHandle(), &SharedContext);

	// Create an OpenGL viewport
	const int32 Width = FMath::TruncToInt(InWindow->GetSizeInScreen().X);
	const int32 Height = FMath::TruncToInt(InWindow->GetSizeInScreen().Y);

	ProjectionMatrix = CreateProjectionMatrix( Width, Height );

	ViewportRect.Right = Width;
	ViewportRect.Bottom = Height;
	ViewportRect.Top = 0;
	ViewportRect.Left = 0;
}

void FSlateOpenGLViewport::Destroy()
{
	RenderingContext.Destroy();
}

void FSlateOpenGLViewport::MakeCurrent()
{
	RenderingContext.MakeCurrent();
}

void FSlateOpenGLViewport::SwapBuffers()
{
	SDL_GL_SwapWindow( RenderingContext.WindowHandle );
}

void FSlateOpenGLViewport::Resize( int32 Width, int32 Height, bool bInFullscreen )
{
	ViewportRect.Right = Width;
	ViewportRect.Bottom = Height;
	bFullscreen = bInFullscreen;
	// Need to create a new projection matrix each time the window is resized
	ProjectionMatrix = CreateProjectionMatrix( Width, Height );
}
