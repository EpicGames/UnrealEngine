// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandaloneRendererPrivate.h"
#include "OpenGL/SlateOpenGLRenderer.h"

#include "SlateOpenGLMac.h"

#include "Mac/MacApplication.h"
#include "Mac/MacWindow.h"
#include "Mac/MacTextInputMethodSystem.h"
#include "Mac/MacPlatformApplicationMisc.h"
#include "Mac/CocoaTextView.h"
#include "Mac/CocoaThread.h"
#include <OpenGL/gl.h>
#include <OpenGL/glext.h>

void LockGLContext(NSOpenGLContext* Context)
{
	[Context lock];
}

void UnlockGLContext(NSOpenGLContext* Context)
{

	[Context unlock];
}

@implementation FSlateOpenGLLayer

- (NSOpenGLPixelFormat *)openGLPixelFormatForDisplayMask:(uint32)Mask
{
	return self.PixelFormat;
}

- (NSOpenGLContext *)openGLContextForPixelFormat:(NSOpenGLPixelFormat *)PixelFormat
{
	return self.Context;
}

- (id)initWithContext:(NSOpenGLContext*)context andPixelFormat:(NSOpenGLPixelFormat*)pixelFormat
{
	self = [super init];
	if (self)
	{
		self.Context = context;
		self.PixelFormat = pixelFormat;
		[self.Context retain];
		[self.PixelFormat retain];
	}
	return self;
}

- (void)dealloc
{
	[self.Context release];
	[self.PixelFormat release];
	[super dealloc];
}

- (BOOL)canDrawInOpenGLContext:(NSOpenGLContext *)context pixelFormat:(NSOpenGLPixelFormat *)pixelFormat forLayerTime:(CFTimeInterval)timeInterval displayTime:(const CVTimeStamp *)timeStamp
{
	BOOL bOK = [super canDrawInOpenGLContext:context pixelFormat:pixelFormat forLayerTime:timeInterval displayTime:timeStamp];
	if ( bOK && context && (self.Context == context) )
	{
		LockGLContext(context);
	}
	// TODO: Should we return NO if !(contex && self.Context == context)?
	return bOK;
}
@end

@implementation FSlateCocoaView

- (CALayer*)makeBackingLayer
{
	return [[FSlateOpenGLLayer alloc] initWithContext:self.Context andPixelFormat:self.PixelFormat];
}

- (id)initWithFrame:(NSRect)frameRect context:(NSOpenGLContext*)context pixelFormat:(NSOpenGLPixelFormat*)pixelFormat
{
	self = [super initWithFrame:frameRect];
	if (self)
	{
		self.Context = context;
		self.PixelFormat = pixelFormat;
		[self.Context retain];
		[self.PixelFormat retain];

		Framebuffer = 0;
		Renderbuffer = 0;
	}
	return self;
}

-(void)dealloc
{
	[self.Context release];
	[self.PixelFormat release];
	[super dealloc];
}

- (void)drawRect:(NSRect)dirtyRect
{
	if (Framebuffer && [(FCocoaWindow*)[self window] isRenderInitialized] && ViewportRect.IsValid())
	{
		const float DPIScaleFactor = FPlatformApplicationMisc::IsHighDPIModeEnabled() ? self.window.backingScaleFactor : 1.0f;
		int32 CurrentReadFramebuffer = 0;
		glGetIntegerv(GL_READ_FRAMEBUFFER_BINDING, &CurrentReadFramebuffer);
		glBindFramebuffer(GL_READ_FRAMEBUFFER, Framebuffer);
		glReadBuffer(GL_COLOR_ATTACHMENT0);
		glBlitFramebuffer(0, 0, ViewportRect.Right, ViewportRect.Bottom, 0, 0, self.frame.size.width * DPIScaleFactor, self.frame.size.height * DPIScaleFactor, GL_COLOR_BUFFER_BIT, GL_NEAREST);
		CHECK_GL_ERRORS;
		glBindFramebuffer(GL_READ_FRAMEBUFFER, CurrentReadFramebuffer);
	}
	else
	{
		glClearColor(0.0f, 0.0f, 0.0f, 0.0f);
		glClear(GL_COLOR_BUFFER_BIT);
	}
	UnlockGLContext(self.Context);
}

- (BOOL)isOpaque
{
	return YES;
}

- (BOOL)mouseDownCanMoveWindow
{
	return YES;
}

@end

static void MacOpenGLContextReconfigurationCallBack(CGDirectDisplayID Display, CGDisplayChangeSummaryFlags Flags, void* UserInfo)
{
	FSlateOpenGLContext* Context = (FSlateOpenGLContext*)UserInfo;
	if (Context)
	{
		Context->bNeedsUpdate = true;
	}
}

FSlateOpenGLContext::FSlateOpenGLContext()
:	View(NULL)
,	PixelFormat(NULL)
,	Context(NULL)
,	bNeedsUpdate(false)
{
}

FSlateOpenGLContext::~FSlateOpenGLContext()
{
	Destroy();
}

void FSlateOpenGLContext::Initialize(void* InWindow, const FSlateOpenGLContext* SharedContext)
{
	NSOpenGLPixelFormatAttribute Attributes[] =
	{
		kCGLPFAAccelerated,
		kCGLPFANoRecovery,
		kCGLPFASupportsAutomaticGraphicsSwitching,
		kCGLPFADoubleBuffer,
		kCGLPFAColorSize,
		32,
		0
	};

	PixelFormat = [[NSOpenGLPixelFormat alloc] initWithAttributes:Attributes];
	Context = [[NSOpenGLContext alloc] initWithFormat:PixelFormat shareContext:SharedContext ? SharedContext->Context : NULL];

	NSWindow* Window = (NSWindow*)InWindow;
	if (Window)
	{
		const NSRect ViewRect = NSMakeRect(0, 0, Window.frame.size.width, Window.frame.size.height);
		View = [[FSlateCocoaView alloc] initWithFrame:ViewRect context:Context pixelFormat:PixelFormat];
		[View setAutoresizingMask:NSViewWidthSizable | NSViewHeightSizable];
		if (FPlatformApplicationMisc::IsHighDPIModeEnabled())
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
				[View setWantsBestResolutionOpenGLSurface : YES];
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		MainThreadCall(^{
			[View setWantsLayer:YES];
			[Window setContentView:View];

			[[Window standardWindowButton:NSWindowCloseButton] setAction:@selector(performClose:)];
		});

		View.layer.magnificationFilter = kCAFilterNearest;
		View.layer.minificationFilter = kCAFilterNearest;
	}

	[Context update];
	MakeCurrent();

	CGDisplayRegisterReconfigurationCallback(&MacOpenGLContextReconfigurationCallBack, this);
}

void FSlateOpenGLContext::Destroy()
{
	if (View)
	{
		LockGLContext(Context);
		NSOpenGLContext* Current = [NSOpenGLContext currentContext];
		[Context makeCurrentContext];
		FSlateCocoaView* SlateView = ((FSlateCocoaView*)View);
		if (SlateView->Framebuffer)
		{
			glDeleteFramebuffers(1, &SlateView->Framebuffer);
			SlateView->Framebuffer = 0;
		}
		if (SlateView->Renderbuffer)
		{
			glDeleteRenderbuffers(1, &SlateView->Renderbuffer);
			SlateView->Renderbuffer = 0;
		}
		[Current makeCurrentContext];
		
		[View release];
		View = NULL;

		// PixelFormat and Context are released by View
		CGDisplayRemoveReconfigurationCallback(&MacOpenGLContextReconfigurationCallBack, this);

		[PixelFormat release];
		[Context clearDrawable];
		UnlockGLContext(Context);
		[Context release];
		PixelFormat = NULL;
		Context = NULL;
	}
}

void FSlateOpenGLContext::MakeCurrent()
{
	if (bNeedsUpdate)
	{
		[Context update];
		bNeedsUpdate = false;
	}
	[Context makeCurrentContext];
}
