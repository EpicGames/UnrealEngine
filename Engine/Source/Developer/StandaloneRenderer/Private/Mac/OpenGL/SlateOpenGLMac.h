// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "StandaloneRendererPrivate.h"
#include "OpenGL/SlateOpenGLRenderer.h"
#include "Mac/MacWindow.h"
#include "Mac/MacTextInputMethodSystem.h"
#include "Mac/CocoaTextView.h"
#include <OpenGL/gl.h>
#include <OpenGL/glext.h>

void LockGLContext(NSOpenGLContext* Context);
void UnlockGLContext(NSOpenGLContext* Context);

@interface FSlateOpenGLLayer : NSOpenGLLayer
@property (assign) NSOpenGLContext* Context;
@property (assign) NSOpenGLPixelFormat* PixelFormat;
@end

@interface FSlateCocoaView : FCocoaTextView
{
@public
	GLuint Framebuffer;
	GLuint Renderbuffer;
	FSlateRect ViewportRect;
}
@property (assign) NSOpenGLContext* Context;
@property (assign) NSOpenGLPixelFormat* PixelFormat;
@end
