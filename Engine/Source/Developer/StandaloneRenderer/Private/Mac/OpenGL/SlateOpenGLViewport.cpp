// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandaloneRendererPrivate.h"
#include "OpenGL/SlateOpenGLRenderer.h"
#include "Mac/CocoaThread.h"
#include "SlateOpenGLMac.h"
#include "Widgets/SWindow.h"

#include "Mac/MacWindow.h"

FSlateOpenGLViewport::FSlateOpenGLViewport()
{
}

void FSlateOpenGLViewport::Initialize(TSharedRef<SWindow> InWindow, const FSlateOpenGLContext& SharedContext)
{
	TSharedRef<FGenericWindow> NativeWindow = InWindow->GetNativeWindow().ToSharedRef();
	RenderingContext.Initialize(NativeWindow->GetOSWindowHandle(), &SharedContext);

	const int32 Width = FMath::TruncToInt(InWindow->GetSizeInScreen().X);
	const int32 Height = FMath::TruncToInt(InWindow->GetSizeInScreen().Y);

	ViewportRect.Top = 0;
	ViewportRect.Left = 0;

	Resize(Width, Height, false);
}

void FSlateOpenGLViewport::Destroy()
{
	RenderingContext.Destroy();
}

void FSlateOpenGLViewport::MakeCurrent()
{
	LockGLContext(RenderingContext.Context); // NOTE: We assume here that SwapBuffers will always be called after Viewport->MakeCurrent
	RenderingContext.MakeCurrent();
	glBindFramebuffer(GL_FRAMEBUFFER, ((FSlateCocoaView*)RenderingContext.View)->Framebuffer);
}

void FSlateOpenGLViewport::SwapBuffers()
{
	glFlushRenderAPPLE();
	[(FCocoaWindow*)[RenderingContext.View window] startRendering];
	MainThreadCall(^{
		[RenderingContext.View setNeedsDisplay:YES];
	}, false);
	glBindFramebuffer(GL_FRAMEBUFFER, ((FSlateCocoaView*)RenderingContext.View)->Framebuffer);
	UnlockGLContext(RenderingContext.Context);
}

void FSlateOpenGLViewport::Resize(int32 Width, int32 Height, bool bInFullscreen)
{
	ViewportRect.Right = Width;
	ViewportRect.Bottom = Height;

	// Need to create a new projection matrix each time the window is resized
	ProjectionMatrix = CreateProjectionMatrix(Width, Height);

	if (RenderingContext.Context && RenderingContext.View && Width > 0 && Height > 0)
	{
		LockGLContext(RenderingContext.Context);
		((FSlateCocoaView*)RenderingContext.View)->ViewportRect = ViewportRect;
		
		GLuint& Framebuffer = ((FSlateCocoaView*)RenderingContext.View)->Framebuffer;
		GLuint& Renderbuffer = ((FSlateCocoaView*)RenderingContext.View)->Renderbuffer;
		
		if (Framebuffer == 0)
		{
			glGenFramebuffers(1, &Framebuffer);
			check(Framebuffer);
		}
		
		if (Renderbuffer == 0)
		{
			glGenRenderbuffers(1, &Renderbuffer);
			check(Renderbuffer);
		}
		
		int32 CurrentDrawFramebuffer = 0;
		int32 CurrentReadFramebuffer = 0;
		glGetIntegerv(GL_DRAW_FRAMEBUFFER_BINDING, &CurrentDrawFramebuffer);
		glGetIntegerv(GL_READ_FRAMEBUFFER_BINDING, &CurrentReadFramebuffer);
		
		glBindRenderbuffer(GL_RENDERBUFFER, Renderbuffer);
		glRenderbufferStorage(GL_RENDERBUFFER, GL_RGB, Width, Height);
		glBindRenderbuffer(GL_RENDERBUFFER, 0);
		
		glBindFramebuffer(GL_FRAMEBUFFER, Framebuffer);
		glFramebufferRenderbuffer(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0, GL_RENDERBUFFER, Renderbuffer);
		
		glBindFramebuffer(GL_DRAW_FRAMEBUFFER, CurrentDrawFramebuffer);
		glBindFramebuffer(GL_READ_FRAMEBUFFER, CurrentReadFramebuffer);

		[RenderingContext.Context update];
		UnlockGLContext(RenderingContext.Context);
	}
}
