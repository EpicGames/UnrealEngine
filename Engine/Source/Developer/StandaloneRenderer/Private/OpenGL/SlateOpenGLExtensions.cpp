// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenGL/SlateOpenGLExtensions.h"

#include "HAL/Platform.h"

#if !PLATFORM_USES_GLES

PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

#if PLATFORM_WINDOWS
// Buffers
PFNGLGENBUFFERSARBPROC					glGenBuffers				= NULL;			
PFNGLBINDBUFFERARBPROC					glBindBuffer				= NULL;	
PFNGLBUFFERDATAARBPROC					glBufferData				= NULL;	
PFNGLDELETEBUFFERSARBPROC				glDeleteBuffers				= NULL;
PFNGLMAPBUFFERARBPROC					glMapBuffer					= NULL;
PFNGLUNMAPBUFFERARBPROC					glUnmapBuffer				= NULL;
PFNGLDRAWRANGEELEMENTSPROC				glDrawRangeElements			= NULL;

// Blending/texturing
PFNGLBLENDEQUATIONPROC					glBlendEquation				= NULL;
PFNGLACTIVETEXTUREARBPROC				glActiveTexture				= NULL;

// Shaders
PFNGLCREATESHADERPROC				glCreateShader				= NULL;
PFNGLSHADERSOURCEPROC				glShaderSource				= NULL;
PFNGLCOMPILESHADERPROC				glCompileShader				= NULL;
PFNGLGETSHADERINFOLOGPROC			glGetShaderInfoLog			= NULL;
PFNGLCREATEPROGRAMPROC				glCreateProgram				= NULL;
PFNGLATTACHSHADERPROC				glAttachShader				= NULL;
PFNGLDETACHSHADERPROC				glDetachShader				= NULL;
PFNGLLINKPROGRAMPROC				glLinkProgram				= NULL;
PFNGLGETPROGRAMINFOLOGPROC			glGetProgramInfoLog			= NULL;
PFNGLUSEPROGRAMPROC					glUseProgram				= NULL;
PFNGLDELETESHADERPROC				glDeleteShader				= NULL;
PFNGLDELETEPROGRAMPROC				glDeleteProgram				= NULL;
PFNGLGETSHADERIVPROC				glGetShaderiv				= NULL;
PFNGLGETPROGRAMIVPROC				glGetProgramiv				= NULL;
PFNGLGETUNIFORMLOCATIONPROC			glGetUniformLocation		= NULL;
PFNGLUNIFORM1FPROC					glUniform1f					= NULL;
PFNGLUNIFORM2FPROC					glUniform2f					= NULL;
PFNGLUNIFORM3FPROC					glUniform3f					= NULL;
PFNGLUNIFORM3FVPROC					glUniform3fv				= NULL;
PFNGLUNIFORM4FPROC					glUniform4f					= NULL;
PFNGLUNIFORM4FVPROC					glUniform4fv				= NULL;
PFNGLUNIFORM1IPROC					glUniform1i					= NULL;
PFNGLUNIFORMMATRIX4FVPROC			glUniformMatrix4fv			= NULL;
PFNGLVERTEXATTRIBPOINTERPROC		glVertexAttribPointer		= NULL;
PFNGLBINDATTRIBLOCATIONPROC			glBindAttribLocation		= NULL;
PFNGLENABLEVERTEXATTRIBARRAYPROC	glEnableVertexAttribArray	= NULL;
PFNGLDISABLEVERTEXATTRIBARRAYPROC	glDisableVertexAttribArray	= NULL;
PFNGLGETACTIVEATTRIBPROC			glGetActiveAttrib			= NULL;
PFNGLGETACTIVEUNIFORMPROC			glGetActiveUniform			= NULL;
PFNGLGETATTACHEDSHADERSPROC			glGetAttachedShaders		= NULL;
PFNGLGETATTRIBLOCATIONPROC			glGetAttribLocation			= NULL;

PFNWGLCREATECONTEXTATTRIBSARBPROC	wglCreateContextAttribsARB	= NULL;

PFNGLDEBUGMESSAGECALLBACKARBPROC 	glDebugMessageCallback 		= NULL;
PFNGLBINDVERTEXARRAYPROC 			glBindVertexArray 			= NULL;
PFNGLDELETEVERTEXARRAYSPROC 		glDeleteVertexArrays 		= NULL;
PFNGLGENVERTEXARRAYSPROC 			glGenVertexArrays 			= NULL;
PFNGLISVERTEXARRAYPROC 				glIsVertexArray 			= NULL;

#elif PLATFORM_LINUX

#define DEFINE_GL_ENTRYPOINTS(Type,Func) Type Func = NULL;
ENUM_GL_ENTRYPOINTS(DEFINE_GL_ENTRYPOINTS);

#endif

bool bOpenGLExtensionsLoaded;

/**
 * Loads all OpenGL extensions.  
 * @todo UE: For now we assume this cannot fail                   
 */
void LoadOpenGLExtensions()
{
#if PLATFORM_WINDOWS
	if( !bOpenGLExtensionsLoaded )
	{
		//@todo UE: Assumes the video card supports these for now.
		bOpenGLExtensionsLoaded = true;
		glGenBuffers				=(PFNGLGENBUFFERSARBPROC)			wglGetProcAddress("glGenBuffers");
		glBindBuffer				=(PFNGLBINDBUFFERARBPROC)			wglGetProcAddress("glBindBuffer");
		glBufferData				=(PFNGLBUFFERDATAARBPROC)			wglGetProcAddress("glBufferData");
		glDeleteBuffers				=(PFNGLDELETEBUFFERSARBPROC)		wglGetProcAddress("glDeleteBuffers");
		glMapBuffer					=(PFNGLMAPBUFFERARBPROC)			wglGetProcAddress("glMapBuffer");
		glUnmapBuffer				=(PFNGLUNMAPBUFFERARBPROC)			wglGetProcAddress("glUnmapBuffer");
		glDrawRangeElements			=(PFNGLDRAWRANGEELEMENTSPROC)		wglGetProcAddress("glDrawRangeElements");
		glBlendEquation				=(PFNGLBLENDEQUATIONPROC)			wglGetProcAddress("glBlendEquation");
		glActiveTexture				=(PFNGLACTIVETEXTUREARBPROC)		wglGetProcAddress("glActiveTexture");

		glCreateShader				=(PFNGLCREATESHADERPROC)			wglGetProcAddress("glCreateShader");
		glShaderSource				=(PFNGLSHADERSOURCEPROC)			wglGetProcAddress("glShaderSource");
		glCompileShader				=(PFNGLCOMPILESHADERPROC)			wglGetProcAddress("glCompileShader");
		glGetShaderInfoLog			=(PFNGLGETSHADERINFOLOGPROC)		wglGetProcAddress("glGetShaderInfoLog");
		glCreateProgram				=(PFNGLCREATEPROGRAMPROC)			wglGetProcAddress("glCreateProgram");
		glAttachShader				=(PFNGLATTACHSHADERPROC)			wglGetProcAddress("glAttachShader");
		glDetachShader				=(PFNGLDETACHSHADERPROC)			wglGetProcAddress("glDetachShader");
		glLinkProgram				=(PFNGLLINKPROGRAMPROC)				wglGetProcAddress("glLinkProgram");
		glGetProgramInfoLog			=(PFNGLGETPROGRAMINFOLOGPROC)		wglGetProcAddress("glGetProgramInfoLog");
		glUseProgram				=(PFNGLUSEPROGRAMPROC)				wglGetProcAddress("glUseProgram");
		glDeleteShader				=(PFNGLDELETESHADERPROC)			wglGetProcAddress("glDeleteShader");
		glDeleteProgram				=(PFNGLDELETEPROGRAMPROC)			wglGetProcAddress("glDeleteProgram");
		glGetShaderiv				=(PFNGLGETPROGRAMIVPROC)			wglGetProcAddress("glGetShaderiv");
		glGetProgramiv				=(PFNGLGETPROGRAMIVPROC)			wglGetProcAddress("glGetProgramiv");
		glGetUniformLocation		=(PFNGLGETUNIFORMLOCATIONPROC)		wglGetProcAddress("glGetUniformLocation");
		glUniform1f					=(PFNGLUNIFORM1FPROC)				wglGetProcAddress("glUniform1f");
		glUniform2f					=(PFNGLUNIFORM2FPROC)				wglGetProcAddress("glUniform2f");
		glUniform3f					=(PFNGLUNIFORM3FPROC)				wglGetProcAddress("glUniform3f");
		glUniform3fv				=(PFNGLUNIFORM3FVPROC)				wglGetProcAddress("glUniform3fv");
		glUniform4f					=(PFNGLUNIFORM4FPROC)				wglGetProcAddress("glUniform4f");
		glUniform4fv				=(PFNGLUNIFORM4FVPROC)				wglGetProcAddress("glUniform4fv");
		glUniform1i					=(PFNGLUNIFORM1IPROC)				wglGetProcAddress("glUniform1i");
		glUniformMatrix4fv			=(PFNGLUNIFORMMATRIX4FVPROC)		wglGetProcAddress("glUniformMatrix4fv");
		glVertexAttribPointer       =(PFNGLVERTEXATTRIBPOINTERPROC)		wglGetProcAddress("glVertexAttribPointer");
		glBindAttribLocation		=(PFNGLBINDATTRIBLOCATIONPROC)		wglGetProcAddress("glBindAttribLocation");	
		glEnableVertexAttribArray	=(PFNGLENABLEVERTEXATTRIBARRAYPROC)	wglGetProcAddress("glEnableVertexAttribArray");
		glDisableVertexAttribArray	=(PFNGLDISABLEVERTEXATTRIBARRAYPROC)wglGetProcAddress("glDisableVertexAttribArray");
		glGetActiveAttrib			=(PFNGLGETACTIVEATTRIBPROC)			wglGetProcAddress("glGetActiveAttrib");
		glGetActiveUniform			=(PFNGLGETACTIVEUNIFORMPROC)		wglGetProcAddress("glGetActiveUniform");
		glGetAttachedShaders		=(PFNGLGETATTACHEDSHADERSPROC)		wglGetProcAddress("glGetAttachedShaders");
		glGetAttribLocation			=(PFNGLGETATTRIBLOCATIONPROC)		wglGetProcAddress("glGetAttribLocation");
	}
#elif PLATFORM_LINUX
#define GET_GL_ENTRYPOINTS(Type,Func) { Func = reinterpret_cast<Type>(SDL_GL_GetProcAddress(#Func)); if (NULL == Func) { UE_LOGF(LogInit, Fatal, "Failed to find entry point for %ls", TEXT(#Func)); }}
	ENUM_GL_ENTRYPOINTS(GET_GL_ENTRYPOINTS);
#endif
	// If extensions are needed for your platform add support for them here 
}

PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

#endif // !PLATFORM_USES_GLES
