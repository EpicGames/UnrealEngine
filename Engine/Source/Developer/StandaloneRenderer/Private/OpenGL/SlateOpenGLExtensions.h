// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "StandaloneRendererPlatformHeaders.h"

#if !PLATFORM_USES_GLES

#if PLATFORM_WINDOWS
// buffers
extern PFNGLGENBUFFERSARBPROC				glGenBuffers;				
extern PFNGLBINDBUFFERARBPROC				glBindBuffer;	
extern PFNGLBUFFERDATAARBPROC				glBufferData;	
extern PFNGLDELETEBUFFERSARBPROC			glDeleteBuffers;
extern PFNGLMAPBUFFERARBPROC				glMapBuffer;
extern PFNGLUNMAPBUFFERARBPROC				glUnmapBuffer;
extern PFNGLDRAWRANGEELEMENTSPROC			glDrawRangeElements;

// Blending / texturing
extern PFNGLBLENDEQUATIONPROC				glBlendEquation;
extern PFNGLACTIVETEXTUREARBPROC			glActiveTexture;

// shaders
extern PFNGLCREATESHADERPROC				glCreateShader;
extern PFNGLSHADERSOURCEPROC				glShaderSource;
extern PFNGLCOMPILESHADERPROC				glCompileShader;
extern PFNGLGETSHADERINFOLOGPROC			glGetShaderInfoLog;
extern PFNGLCREATEPROGRAMPROC				glCreateProgram;
extern PFNGLATTACHSHADERPROC				glAttachShader;
extern PFNGLDETACHSHADERPROC				glDetachShader;
extern PFNGLLINKPROGRAMPROC					glLinkProgram;
extern PFNGLGETPROGRAMINFOLOGPROC			glGetProgramInfoLog;
extern PFNGLUSEPROGRAMPROC					glUseProgram;
extern PFNGLDELETESHADERPROC				glDeleteShader;
extern PFNGLDELETEPROGRAMPROC				glDeleteProgram;
extern PFNGLGETSHADERIVPROC					glGetShaderiv;
extern PFNGLGETPROGRAMIVPROC				glGetProgramiv;
extern PFNGLGETUNIFORMLOCATIONPROC			glGetUniformLocation;
extern PFNGLUNIFORM1FPROC					glUniform1f;
extern PFNGLUNIFORM2FPROC					glUniform2f;
extern PFNGLUNIFORM3FPROC					glUniform3f;
extern PFNGLUNIFORM3FVPROC					glUniform3fv;
extern PFNGLUNIFORM4FPROC					glUniform4f;
extern PFNGLUNIFORM4FVPROC					glUniform4fv;
extern PFNGLUNIFORM1IPROC					glUniform1i;
extern PFNGLUNIFORMMATRIX4FVPROC			glUniformMatrix4fv;
extern PFNGLVERTEXATTRIBPOINTERPROC			glVertexAttribPointer;
extern PFNGLBINDATTRIBLOCATIONPROC			glBindAttribLocation;
extern PFNGLENABLEVERTEXATTRIBARRAYPROC		glEnableVertexAttribArray;
extern PFNGLDISABLEVERTEXATTRIBARRAYPROC	glDisableVertexAttribArray;
extern PFNGLGETACTIVEATTRIBPROC				glGetActiveAttrib;
extern PFNGLGETACTIVEUNIFORMPROC			glGetActiveUniform;
extern PFNGLGETATTACHEDSHADERSPROC			glGetAttachedShaders;
extern PFNGLGETATTRIBLOCATIONPROC			glGetAttribLocation;

extern PFNWGLCREATECONTEXTATTRIBSARBPROC	wglCreateContextAttribsARB;

extern PFNGLDEBUGMESSAGECALLBACKARBPROC 	glDebugMessageCallback;
extern PFNGLBINDVERTEXARRAYPROC 			glBindVertexArray;
extern PFNGLDELETEVERTEXARRAYSPROC 			glDeleteVertexArrays;
extern PFNGLGENVERTEXARRAYSPROC 			glGenVertexArrays;
extern PFNGLISVERTEXARRAYPROC 				glIsVertexArray;

#elif PLATFORM_LINUX

#define ENUM_GL_ENTRYPOINTS(EnumMacro) \
	EnumMacro(PFNGLBINDTEXTUREPROC,glBindTexture) \
	EnumMacro(PFNGLBLENDFUNCPROC,glBlendFunc) \
	EnumMacro(PFNGLDELETETEXTURESPROC,glDeleteTextures) \
	EnumMacro(PFNGLDISABLEPROC,glDisable) \
	EnumMacro(PFNGLENABLEPROC,glEnable) \
	EnumMacro(PFNGLGENTEXTURESPROC,glGenTextures) \
	EnumMacro(PFNGLGETERRORPROC,glGetError) \
	EnumMacro(PFNGLPIXELSTOREIPROC,glPixelStorei) \
	EnumMacro(PFNGLPOLYGONMODEPROC,glPolygonMode) \
	EnumMacro(PFNGLSTENCILFUNCPROC,glStencilFunc) \
	EnumMacro(PFNGLSTENCILMASKPROC,glStencilMask) \
	EnumMacro(PFNGLSTENCILOPPROC,glStencilOp) \
	EnumMacro(PFNGLTEXIMAGE2DPROC,glTexImage2D) \
	EnumMacro(PFNGLTEXSUBIMAGE2DPROC,glTexSubImage2D) \
	EnumMacro(PFNGLTEXPARAMETERIPROC,glTexParameteri) \
	EnumMacro(PFNGLVIEWPORTPROC,glViewport) \
	EnumMacro(PFNGLBLENDEQUATIONPROC,glBlendEquation) \
	EnumMacro(PFNGLDRAWRANGEELEMENTSPROC,glDrawRangeElements) \
	EnumMacro(PFNGLACTIVETEXTUREPROC,glActiveTexture) \
	EnumMacro(PFNGLBINDBUFFERPROC,glBindBuffer) \
	EnumMacro(PFNGLDELETEBUFFERSPROC,glDeleteBuffers) \
	EnumMacro(PFNGLGENBUFFERSPROC,glGenBuffers) \
	EnumMacro(PFNGLBUFFERDATAPROC,glBufferData) \
	EnumMacro(PFNGLMAPBUFFERPROC,glMapBuffer) \
	EnumMacro(PFNGLUNMAPBUFFERPROC,glUnmapBuffer) \
	EnumMacro(PFNGLATTACHSHADERPROC,glAttachShader) \
	EnumMacro(PFNGLBINDATTRIBLOCATIONPROC,glBindAttribLocation) \
	EnumMacro(PFNGLCOMPILESHADERPROC,glCompileShader) \
	EnumMacro(PFNGLCREATEPROGRAMPROC,glCreateProgram) \
	EnumMacro(PFNGLCREATESHADERPROC,glCreateShader) \
	EnumMacro(PFNGLDELETEPROGRAMPROC,glDeleteProgram) \
	EnumMacro(PFNGLDELETESHADERPROC,glDeleteShader) \
	EnumMacro(PFNGLENABLEVERTEXATTRIBARRAYPROC,glEnableVertexAttribArray) \
	EnumMacro(PFNGLGETPROGRAMIVPROC,glGetProgramiv) \
	EnumMacro(PFNGLGETPROGRAMINFOLOGPROC,glGetProgramInfoLog) \
	EnumMacro(PFNGLGETSHADERIVPROC,glGetShaderiv) \
	EnumMacro(PFNGLGETSHADERINFOLOGPROC,glGetShaderInfoLog) \
	EnumMacro(PFNGLGETUNIFORMLOCATIONPROC,glGetUniformLocation) \
	EnumMacro(PFNGLLINKPROGRAMPROC,glLinkProgram) \
	EnumMacro(PFNGLSHADERSOURCEPROC,glShaderSource) \
	EnumMacro(PFNGLUSEPROGRAMPROC,glUseProgram) \
	EnumMacro(PFNGLUNIFORM2FPROC,glUniform2f) \
	EnumMacro(PFNGLUNIFORM4FPROC,glUniform4f) \
	EnumMacro(PFNGLUNIFORM1IPROC,glUniform1i) \
	EnumMacro(PFNGLUNIFORM4FVPROC,glUniform4fv) \
	EnumMacro(PFNGLUNIFORMMATRIX4FVPROC,glUniformMatrix4fv) \
	EnumMacro(PFNGLVERTEXATTRIBPOINTERPROC,glVertexAttribPointer) \
	EnumMacro(PFNGLDEBUGMESSAGECALLBACKARBPROC,glDebugMessageCallbackARB) \
	EnumMacro(PFNGLDEBUGMESSAGECONTROLARBPROC,glDebugMessageControlARB) \
	EnumMacro(PFNGLBINDVERTEXARRAYPROC,glBindVertexArray) \
	EnumMacro(PFNGLDELETEVERTEXARRAYSPROC,glDeleteVertexArrays) \
	EnumMacro(PFNGLGENVERTEXARRAYSPROC,glGenVertexArrays) \
	EnumMacro(PFNGLISVERTEXARRAYPROC,glIsVertexArray) \
	EnumMacro(PFNGLSCISSORPROC,glScissor) \

#define DECLARE_GL_ENTRYPOINTS(Type,Func) extern Type Func;
ENUM_GL_ENTRYPOINTS(DECLARE_GL_ENTRYPOINTS);

#endif

void LoadOpenGLExtensions();

#endif //!PLATFORM_IOS
