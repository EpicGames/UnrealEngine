// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenGL/SlateOpenGLIndexBuffer.h"
#include "Rendering/RenderingCommon.h"
#include "OpenGL/SlateOpenGLExtensions.h"


FSlateOpenGLIndexBuffer::FSlateOpenGLIndexBuffer()
	: MaxNumIndices(0)
	, BufferID(0)
{

}

FSlateOpenGLIndexBuffer::~FSlateOpenGLIndexBuffer()
{
	DestroyBuffer();
}

/** 
 * Initializes the index buffers resource if needed
 */
void FSlateOpenGLIndexBuffer::ConditionalCreateBuffer()
{
	// Only generate the buffer if we don't have a valid one
	if( !IsValid() )
	{
		glGenBuffers( 1, &BufferID );
	}
}

/** 
 * Resizes the buffer to the passed in number of indices.  
 * Preserves internal data
 */
void FSlateOpenGLIndexBuffer::ResizeBuffer( uint32 NumIndices )
{
	// Only resize if the index buffer cant provide the number of indices  requested
	if( NumIndices > MaxNumIndices )
	{
		// Determine the current buffer size so we can save off the current indices
		uint32 CurrentBufferSize = MaxNumIndices*sizeof(SlateIndex);
		uint8 *SavedIndices = NULL;

		// If there are any indices at all save them off now
		if( MaxNumIndices > 0 )
		{
			void *Indices = Lock( 0 );

			SavedIndices = new uint8[CurrentBufferSize];
			FMemory::Memcpy( SavedIndices, Indices, CurrentBufferSize );

			Unlock();

			// Destroy the current buffer.  It needs to be recreated with a larger size
			DestroyBuffer();
		}

		// Calculate the new buffer size
		BufferSize = NumIndices*sizeof(SlateIndex);
		// Create the index buffer
		ConditionalCreateBuffer();

		// Bind the buffer so we can give it data
		Bind();

		MaxNumIndices = NumIndices;

		// Set the index buffer's size
		glBufferData( GL_ELEMENT_ARRAY_BUFFER, BufferSize, NULL, GL_DYNAMIC_DRAW );
		
		// If there are any saved indices, copy them back now.
		if( SavedIndices )
		{
			void *Indices = Lock( 0 );

			FMemory::Memcpy( Indices, SavedIndices, CurrentBufferSize );

			Unlock();

			delete[] SavedIndices;
		}
	}
}

/** 
 * Locks the index buffer, returning a pointer to the indices
 */
void* FSlateOpenGLIndexBuffer::Lock( uint32 FirstIndex )
{
	// Bind the index buffer so we can access its data
	Bind();

	// Map the buffer data
	// Calling glBufferData here is the same as a discard.  We avoid a pipeline flush
	glBufferData( GL_ELEMENT_ARRAY_BUFFER, BufferSize, NULL, GL_DYNAMIC_DRAW );
	void* Data = glMapBuffer( GL_ELEMENT_ARRAY_BUFFER, GL_WRITE_ONLY );
	check(Data);

	uint32 Offset = FirstIndex * sizeof(SlateIndex);
	return (void*)((SlateIndex*)Data + FirstIndex);
}

/** 
 * Unlocks the buffer.  Pointers to buffer data will no longer be valid after this call
 */
void FSlateOpenGLIndexBuffer::Unlock()
{
	Bind();
	glUnmapBuffer( GL_ELEMENT_ARRAY_BUFFER );
}

/** 
 * Binds the buffer so it can be accessed
 */
void FSlateOpenGLIndexBuffer::Bind()
{
	glBindBuffer( GL_ELEMENT_ARRAY_BUFFER, BufferID );
}

/** 
 * Releases the index buffers resource. 
 */
void FSlateOpenGLIndexBuffer::DestroyBuffer()
{
	if( IsValid() )
	{
		glDeleteBuffers( 1, &BufferID );
		BufferID = 0;
	}
}
