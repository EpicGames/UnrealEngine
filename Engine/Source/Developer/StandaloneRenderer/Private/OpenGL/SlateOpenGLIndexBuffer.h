// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "StandaloneRendererPlatformHeaders.h"

/**
 * An OpenGL index buffer                   
 */
class FSlateOpenGLIndexBuffer 
{
public:
	FSlateOpenGLIndexBuffer();
	~FSlateOpenGLIndexBuffer();

	/** 
	 * Releases the index buffers resource. 
	 */
	void DestroyBuffer();

	/** 
	 * Returns the maximum number of indices that can be used by this buffer 
	 */
	uint32 GetMaxNumIndices() const { return MaxNumIndices; }

	/** 
	 * Resizes the buffer to the passed in number of indices.  
	 * Preserves internal data
	 */
	void ResizeBuffer( uint32 NumIndices );

	/** 
	 * Locks the index buffer, returning a pointer to the indices
	 */
	void* Lock( uint32 FirstIndex );

	/** 
	 * Unlocks the buffer.  Pointers to buffer data will no longer be valid after this call
	 */
	void Unlock();

	/** 
	 * Binds the buffer so it can be accessed
	 */
	void Bind();

	/**
	 * Returns true if the buffer is valid and can be used                   
	 */
	bool IsValid() const { return BufferID != 0; }
private:
	/** 
	 * Initializes the index buffers resource if needed
	 */
	void ConditionalCreateBuffer();
private:
	/** The maximum number of indices this buffer can hold */
	GLuint MaxNumIndices;
	/** An OpenGL resource id for this buffer */
	GLuint BufferID;
	/** The size of the buffer in bytes */
	GLuint BufferSize;
	/** Hidden copy methods. */
	FSlateOpenGLIndexBuffer( const FSlateOpenGLIndexBuffer& );
	void operator=(const FSlateOpenGLIndexBuffer& );

};
