// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenGL/SlateOpenGLRenderer.h"
#include "Fonts/FontTypes.h"
#include "Fonts/FontCache.h"
#include "Types/SlateVector2.h"
#include "Widgets/SWindow.h"
#include "OpenGL/SlateOpenGLTextures.h"

#include "OpenGL/SlateOpenGLTextureManager.h"
#include "OpenGL/SlateOpenGLRenderingPolicy.h"
#include "Rendering/ElementBatcher.h"

#if PLATFORM_LINUX
	#include "HAL/PlatformApplicationMisc.h"
#elif PLATFORM_MAC
	#include "Mac/CocoaThread.h"
#endif

class FSlateOpenGLFontAtlasFactory : public ISlateFontAtlasFactory
{
public:
	virtual ~FSlateOpenGLFontAtlasFactory()
	{
	}

	virtual FIntPoint GetAtlasSize(ESlateFontAtlasContentType InContentType) const override
	{
		switch (InContentType)
		{
			case ESlateFontAtlasContentType::Alpha:
				return FIntPoint(GrayscaleTextureSize, GrayscaleTextureSize);
			case ESlateFontAtlasContentType::Color:
				return FIntPoint(ColorTextureSize, ColorTextureSize);
			case ESlateFontAtlasContentType::Msdf:
				return FIntPoint(SdfTextureSize, SdfTextureSize);
			default:
				checkNoEntry();
				// Default to COLOR
				return FIntPoint(ColorTextureSize, ColorTextureSize);
		}
	}

	virtual TSharedRef<FSlateFontAtlas> CreateFontAtlas(ESlateFontAtlasContentType InContentType) const override
	{
		const FIntPoint AtlasSize = GetAtlasSize(InContentType);

		TSharedRef<FSlateFontTextureOpenGL> FontTexture = MakeShareable(new FSlateFontTextureOpenGL(AtlasSize.X, AtlasSize.Y, InContentType));
		FontTexture->CreateFontTexture();

		return FontTexture;
	}

	virtual TSharedPtr<ISlateFontTexture> CreateNonAtlasedTexture(const uint32 InWidth, const uint32 InHeight, ESlateFontAtlasContentType InContentType, const TArray<uint8>& InRawData) const override
	{
		return nullptr;
	}

private:

	/** Size of each font texture, width and height */
	static const uint32 GrayscaleTextureSize = 1024;
	static const uint32 ColorTextureSize = 512;
	static const uint32 SdfTextureSize = 1024;
};

TSharedRef<FSlateFontServices> CreateOpenGLFontServices()
{
	const TSharedRef<FSlateFontCache> FontCache = MakeShareable(new FSlateFontCache(MakeShareable(new FSlateOpenGLFontAtlasFactory), ESlateTextureAtlasThreadId::Game));

	return MakeShareable(new FSlateFontServices(FontCache, FontCache));
}

FSlateOpenGLRenderer::FSlateOpenGLRenderer( const ISlateStyle& InStyle )
	: FSlateRenderer( CreateOpenGLFontServices() )
	, Style( InStyle )
{

	ViewMatrix = FMatrix(	FPlane(1,	0,	0,	0),
							FPlane(0,	1,	0,	0),
							FPlane(0,	0,	1,  0),
							FPlane(0,	0,	0,	1));

#if PLATFORM_LINUX
	FPlatformApplicationMisc::UsingOpenGL();
#endif // PLATFORM_LINUX
}

FSlateOpenGLRenderer::~FSlateOpenGLRenderer()
{
}

/** Returns a draw buffer that can be used by Slate windows to draw window elements */
FSlateDrawBuffer& FSlateOpenGLRenderer::AcquireDrawBuffer()
{
	ensureMsgf(!DrawBuffer.IsLocked(), TEXT("The DrawBuffer is already locked. Make sure to ReleaseDrawBuffer the DrawBuffer"));
	DrawBuffer.Lock();

	// Clear out the buffer each time its accessed
	DrawBuffer.ClearBuffer();

	return DrawBuffer;
}

void FSlateOpenGLRenderer::ReleaseDrawBuffer(FSlateDrawBuffer& InWindowDrawBuffer)
{
	ensureMsgf(&DrawBuffer == &InWindowDrawBuffer, TEXT("It release a DrawBuffer that is not a member of the SlateNullRenderer"));
	InWindowDrawBuffer.Unlock();
}

bool FSlateOpenGLRenderer::Initialize()
{
	SharedContext.Initialize( NULL, NULL );

	TextureManager = MakeShareable( new FSlateOpenGLTextureManager );

	RenderingPolicy = MakeShareable( new FSlateOpenGLRenderingPolicy( SlateFontServices.ToSharedRef(), TextureManager.ToSharedRef() ) );

	ElementBatcher = MakeUnique<FSlateElementBatcher>(RenderingPolicy.ToSharedRef());

#if !PLATFORM_USES_GLES
	// Load OpenGL extensions if needed.  Need a current rendering context to do this
	LoadOpenGLExtensions();
#endif

	TextureManager->LoadUsedTextures();

	// Create rendering resources if needed
	RenderingPolicy->ConditionalInitializeResources();

	return true;
}

/** 
 * Creates necessary resources to render a window and sends draw commands to the rendering thread
 *
 * @param WindowDrawBuffer	The buffer containing elements to draw 
 */
void FSlateOpenGLRenderer::DrawWindows( FSlateDrawBuffer& InWindowDrawBuffer )
{
	const TSharedRef<FSlateFontCache> FontCache = SlateFontServices->GetFontCache();

	// Draw each window.  For performance.  All elements are batched before anything is rendered
	const TArray< TSharedRef<FSlateWindowElementList> >& WindowElementLists = InWindowDrawBuffer.GetWindowElementLists();

	for( int32 ListIndex = 0; ListIndex < WindowElementLists.Num(); ++ListIndex )
	{
		FSlateWindowElementList& ElementList = *WindowElementLists[ListIndex];

		if ( ElementList.GetRenderWindow() )
		{
			SWindow* WindowToDraw = ElementList.GetRenderWindow();

			const FVector2D WindowSize = WindowToDraw->GetSizeInScreen();
			if (WindowSize.X > 0 && WindowSize.Y > 0)
			{
				FSlateOpenGLViewport* Viewport = WindowToViewportMap.Find( WindowToDraw );
				check(Viewport);

				//@todo Slate OpenGL: Move this to ResizeViewport
				if( WindowSize.X != Viewport->ViewportRect.Right || WindowSize.Y != Viewport->ViewportRect.Bottom )
				{
					//@todo implement fullscreen
					const bool bFullscreen = false;
					Private_ResizeViewport( UE::Slate::CastToVector2f(WindowSize), *Viewport, bFullscreen );
				}

				Viewport->MakeCurrent();

				// Update texture cache of pending requests before the resources are accessed during batching
				TextureManager->UpdateCache();

				// Batch elements.  Note that we must set the current viewport before doing this so we have a valid rendering context when calling OpenGL functions
				ElementBatcher->AddElements(ElementList);

				// Update the font cache with new text after elements are batched
				FontCache->UpdateCache();
			

				//@ todo Slate: implement for opengl
				bool bRequiresStencilTest = false;

				ElementBatcher->ResetBatches();
			
				FSlateBatchData& BatchData = ElementList.GetBatchData();

				RenderingPolicy->BuildRenderingBuffers( BatchData );

				check(Viewport);

				glViewport( Viewport->ViewportRect.Left, Viewport->ViewportRect.Top, Viewport->ViewportRect.Right, Viewport->ViewportRect.Bottom );

				if (BatchData.GetRenderBatches().Num() > 0)
				{
					// Draw all elements
					RenderingPolicy->DrawElements( ViewMatrix*Viewport->ProjectionMatrix, WindowSize, BatchData.GetRenderBatches() );
				}

				Viewport->SwapBuffers();

				// All elements have been drawn.  Reset all cached data
				ElementBatcher->ResetBatches();
			}
		}
	}

	// flush the cache if needed
	FontCache->ConditionalFlushCache();
	TextureManager->ConditionalFlushCache();

	// Safely release the references now that we are finished rendering with the dynamic brushes
	DynamicBrushesToRemove.Empty();
}



/** Called when a window is destroyed to give the renderer a chance to free resources */
void FSlateOpenGLRenderer::OnWindowDestroyed( const TSharedRef<SWindow>& InWindow )
{
	FSlateOpenGLViewport* Viewport = WindowToViewportMap.Find( &InWindow.Get() );
	if( Viewport )
	{
		Viewport->Destroy();
	}
	WindowToViewportMap.Remove( &InWindow.Get() );

	SharedContext.MakeCurrent();
}

void FSlateOpenGLRenderer::CreateViewport( const TSharedRef<SWindow> InWindow )
{
#if UE_BUILD_DEBUG
	// Ensure a viewport for this window doesnt already exist
	FSlateOpenGLViewport* Viewport = WindowToViewportMap.Find( &InWindow.Get() );
	check(!Viewport);
#endif

	FSlateOpenGLViewport& NewViewport = WindowToViewportMap.Add( &InWindow.Get(), FSlateOpenGLViewport() );
	NewViewport.Initialize( InWindow, SharedContext );
}


void FSlateOpenGLRenderer::RequestResize( const TSharedPtr<SWindow>& InWindow, uint32 NewSizeX, uint32 NewSizeY )
{
	// @todo implement.  Viewports are currently resized in DrawWindows
}

void FSlateOpenGLRenderer::Private_ResizeViewport( FVector2f WindowSize, FSlateOpenGLViewport& InViewport, bool bFullscreen )
{
	uint32 Width = FMath::TruncToInt(WindowSize.X);
	uint32 Height = FMath::TruncToInt(WindowSize.Y);

	InViewport.Resize( Width, Height, bFullscreen );
}

void FSlateOpenGLRenderer::UpdateFullscreenState( const TSharedRef<SWindow> InWindow, uint32 OverrideResX, uint32 OverrideResY )
{
	FSlateOpenGLViewport* Viewport = WindowToViewportMap.Find( &InWindow.Get() );
	 
	if( Viewport )
	{
		bool bFullscreen = IsViewportFullscreen( *InWindow );

		// todo: support Fullscreen modes in OpenGL
//		uint32 ResX = OverrideResX ? OverrideResX : GSystemResolution.ResX;
//		uint32 ResY = OverrideResY ? OverrideResY : GSystemResolution.ResY;

		Private_ResizeViewport( FVector2f( Viewport->ViewportRect.Right, Viewport->ViewportRect.Bottom ), *Viewport, bFullscreen );
	}
}


void FSlateOpenGLRenderer::ReleaseDynamicResource( const FSlateBrush& Brush )
{
	TextureManager->ReleaseDynamicTextureResource( Brush );
}


bool FSlateOpenGLRenderer::GenerateDynamicImageResource(FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& Bytes)
{
	return TextureManager->CreateDynamicTextureResource(ResourceName, Width, Height, Bytes) != NULL;
}

bool FSlateOpenGLRenderer::GenerateDynamicImageResource(FName ResourceName, FSlateTextureDataRef TextureData)
{
	return GenerateDynamicImageResource(ResourceName, TextureData->GetWidth(), TextureData->GetHeight(), TextureData->GetRawBytes());
}


FSlateResourceHandle FSlateOpenGLRenderer::GetResourceHandle(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale)
{
	return TextureManager->GetResourceHandle(Brush, LocalSize, DrawScale);
}

void FSlateOpenGLRenderer::RemoveDynamicBrushResource( TSharedPtr<FSlateDynamicImageBrush> BrushToRemove )
{
	DynamicBrushesToRemove.Add( BrushToRemove );
}


void FSlateOpenGLRenderer::LoadStyleResources( const ISlateStyle& InStyle )
{
	if ( TextureManager.IsValid() )
	{
		TextureManager->LoadStyleResources( InStyle );
	}
}

FSlateUpdatableTexture* FSlateOpenGLRenderer::CreateUpdatableTexture(uint32 Width, uint32 Height)
{
	TArray<uint8> RawData;
	RawData.AddZeroed(Width * Height * 4);
	FSlateOpenGLTexture* NewTexture = new FSlateOpenGLTexture(Width, Height);
	NewTexture->Init(GL_RGBA, RawData);
	return NewTexture;
}

FSlateUpdatableTexture* FSlateOpenGLRenderer::CreateSharedHandleTexture(void* SharedHandle)
{
	FSlateOpenGLTexture* NewTexture = new FSlateOpenGLTexture();
	NewTexture->Init(SharedHandle);
	return NewTexture;
}

void FSlateOpenGLRenderer::ReleaseUpdatableTexture(FSlateUpdatableTexture* Texture)
{
	Texture->Cleanup();
}

ISlateAtlasProvider* FSlateOpenGLRenderer::GetTextureAtlasProvider()
{
	if( TextureManager.IsValid() )
	{
		return TextureManager->GetTextureAtlasProvider();
	}

	return nullptr;
}

FCriticalSection* FSlateOpenGLRenderer::GetResourceCriticalSection()
{
	return &ResourceCriticalSection;
}

void FSlateOpenGLRenderer::RegisterCurrentScene_Impl(FSceneInterface* Scene)
{
	// This is a no-op
}

int32 FSlateOpenGLRenderer::GetCurrentSceneIndex() const
{
	// This is a no-op
	return -1;
}

void FSlateOpenGLRenderer::SetCurrentSceneIndex(int32 InIndex)
{
	// This is a no-op
}

void FSlateOpenGLRenderer::ClearScenes() 
{
	// This is a no-op
}
