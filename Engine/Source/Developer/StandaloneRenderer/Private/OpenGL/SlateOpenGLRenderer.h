// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StandaloneRendererPlatformHeaders.h"
#include "Layout/SlateRect.h"
#include "Textures/SlateShaderResource.h"
#include "Rendering/SlateDrawBuffer.h"
#include "Rendering/SlateRenderer.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

class Error;
class FSlateElementBatcher;
class FSlateOpenGLRenderingPolicy;
class FSlateOpenGLTextureManager;
class FSlateUpdatableTexture;
class ISlateAtlasProvider;
class ISlateStyle;
class SWindow;

// Optionally use 3.2 context on Linux. There's no real need to require this in a standalone application since it only renders Slate UI.
// With this set to 0, StandaloneRenderer will use OpenGL 2.1 on Linux, which is almost universally supported (as of 2015).
#define LINUX_USE_OPENGL_3_2		0

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#define CHECK_GL_ERRORS \
{\
	GLenum Error = glGetError();\
	checkf( Error == GL_NO_ERROR, TEXT("GL error: 0x%x"), Error );\
}
#else
#define CHECK_GL_ERRORS
#endif

/**
 * Representation of an OpenGL context
 */
struct FSlateOpenGLContext
{
#if PLATFORM_WINDOWS
	HWND WindowHandle;
	HDC WindowDC;
	HGLRC Context;
	bool bReleaseWindowOnDestroy;
#elif PLATFORM_MAC
	NSView* View;
	NSOpenGLPixelFormat* PixelFormat;
	NSOpenGLContext* Context;
	bool bNeedsUpdate;
#elif PLATFORM_IOS
	UIWindow* WindowHandle;
	EAGLContext* Context;
#elif PLATFORM_LINUX
	SDL_Window* WindowHandle;
	SDL_GLContext Context;
	bool bReleaseWindowOnDestroy;
#if LINUX_USE_OPENGL_3_2
	GLuint VertexArrayObject; 
#endif // LINUX_USE_OPENGL_3_2

#else
#error "Unknown platform"
#endif

	FSlateOpenGLContext();
	~FSlateOpenGLContext();

	/**
	 * Initializes the context
	 *
	 * @param InWindow			Window owning this context or NULL
	 * @param SharedContext		The OpneGL context to share resources with or NULL
	 */
	void Initialize( void* InWindow, const FSlateOpenGLContext* SharedContext );
	/** Releases the OpenGL context */
	void Destroy();
	/** Sets this context as active */
	void MakeCurrent();
};

/**
 * Representation of an OpenGL viewport
 */
struct FSlateOpenGLViewport
{
	FMatrix ProjectionMatrix;
	FSlateRect ViewportRect;
	FSlateOpenGLContext RenderingContext;
	/** Whether or not we are fullscreen (@todo not implemented) */
	bool bFullscreen;

	FSlateOpenGLViewport();
	~FSlateOpenGLViewport() { Destroy(); }

	/**
	 * Initializes the viewport
	 *
	 * @param InWindow			Window owning this viewport
	 * @param SharedContext		The OpneGL context to share resources with or NULL
	 */
	void Initialize( TSharedRef<SWindow> InWindow, const FSlateOpenGLContext& SharedContext );
	/** Releases the OpenGL context */
	void Destroy();

	/** Sets this viewport's OpenGL context as active */
	void MakeCurrent();
	/** Brings the contents of the back buffer to screen */
	void SwapBuffers();

	/** Updates the OpenGL context and projection matrix on resize */
	void Resize( int32 Width, int32 Height, bool bInFullscreen );

private:

	/**
	 * Creates an orthographic matrix for use in OpenGL.
	 *
	 * @param Width		The width of the viewport
	 * @param Height	The height of the viewport
	 */
	FMatrix CreateProjectionMatrix( uint32 Width, uint32 Height ) const
	{
		// Create ortho projection matrix
		const float Left = 0.0f;
		const float Right = Left+Width;
		const float Top = 0.0f;
		const float Bottom = Top+Height;
		const float ZNear = -100.0f;
		const float ZFar = 100.0f;

		return	FMatrix( FPlane(2.0f/(Right-Left),			0,							0,					0 ),
						 FPlane(0,							2.0f/(Top-Bottom),			0,					0 ),
						 FPlane(0,							0,							1/(ZNear-ZFar),		0 ),
						 FPlane((Left+Right)/(Left-Right),	(Top+Bottom)/(Bottom-Top),	ZNear/(ZNear-ZFar), 1 ) );

	}
};

/**
 * An OpenGL renderer for use in rendering slate elements                   
 */
class FSlateOpenGLRenderer : public FSlateRenderer
{
public:
	STANDALONERENDERER_API FSlateOpenGLRenderer( const ISlateStyle& InStyle );
	~FSlateOpenGLRenderer();

	/** FSlateRenderer interface */
	virtual FSlateDrawBuffer& AcquireDrawBuffer() override;
	virtual void ReleaseDrawBuffer( FSlateDrawBuffer& InWindowDrawBuffer ) override;
	virtual bool Initialize() override;
	virtual void Destroy() override {}
	virtual void DrawWindows( FSlateDrawBuffer& InWindowDrawBuffer ) override;
	virtual void OnWindowDestroyed( const TSharedRef<SWindow>& InWindow ) override;
	virtual void CreateViewport( const TSharedRef<SWindow> InWindow ) override;
	virtual void RequestResize( const TSharedPtr<SWindow>& InWindow, uint32 NewSizeX, uint32 NewSizeY ) override;
	virtual void UpdateFullscreenState( const TSharedRef<SWindow> InWindow, uint32 OverrideResX, uint32 OverrideResY ) override;
	virtual void SetSystemResolution(uint32 Width, uint32 Height) override {}
	virtual void RestoreSystemResolution(const TSharedRef<SWindow> InWindow) override {}
	virtual void ReleaseDynamicResource( const FSlateBrush& Brush ) override;
	virtual bool GenerateDynamicImageResource(FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& Bytes) override;
	virtual bool GenerateDynamicImageResource(FName ResourceName, FSlateTextureDataRef TextureData) override;
	virtual FSlateResourceHandle GetResourceHandle(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale) override;
	virtual void RemoveDynamicBrushResource( TSharedPtr<FSlateDynamicImageBrush> BrushToRemove ) override;
	virtual void LoadStyleResources(const ISlateStyle& Style) override;
	virtual FSlateUpdatableTexture* CreateUpdatableTexture(uint32 Width, uint32 Height) override;
	virtual FSlateUpdatableTexture* CreateSharedHandleTexture(void* SharedHandle) override;
	virtual void ReleaseUpdatableTexture(FSlateUpdatableTexture* Texture) override;
	virtual ISlateAtlasProvider* GetTextureAtlasProvider() override;
	virtual FCriticalSection* GetResourceCriticalSection() override;
	virtual int32 GetCurrentSceneIndex() const override;
	virtual void SetCurrentSceneIndex(int32 InIndex) override;
	virtual void ClearScenes() override;

protected:
	virtual void RegisterCurrentScene_Impl(FSceneInterface* Scene) override;

private:
	/** 
	 * Resizes an OpenGL Viewport
	 *
	 * @param WindowSize    The Size of the window in pixels
	 * @param InViewport    The viewport to resize
	 */
	void Private_ResizeViewport( FVector2f WindowSize, FSlateOpenGLViewport& InViewport, bool bFullscreen );
private:
	/** View matrix to use when rendering */
	FMatrix ViewMatrix;
	/** A mapping of slate windows to OpenGL viewports */
	TMap<const SWindow*, FSlateOpenGLViewport> WindowToViewportMap;
	/** The buffer available to slate for creating draw elements */
	FSlateDrawBuffer DrawBuffer;
	/** The element batcher used to create and batch geometry for each element. */
	TUniquePtr<FSlateElementBatcher> ElementBatcher;
	/** Texture manager for accessing slate textures */
 	TSharedPtr<FSlateOpenGLTextureManager> TextureManager;
	/** The rendering policy to use when drawing elements */
	TSharedPtr<FSlateOpenGLRenderingPolicy> RenderingPolicy;
	/** Slate style used to create textures for rendering */
	const ISlateStyle& Style;
	/** Shared OpenGL context */
	FSlateOpenGLContext SharedContext;
	/** Dynamic image brushes to remove when safe */
	TArray<TSharedPtr<FSlateDynamicImageBrush>> DynamicBrushesToRemove;
	FCriticalSection ResourceCriticalSection;
};
