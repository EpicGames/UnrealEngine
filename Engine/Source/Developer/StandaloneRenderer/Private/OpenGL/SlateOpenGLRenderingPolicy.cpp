// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenGL/SlateOpenGLRenderingPolicy.h"
#include "Misc/Paths.h"

#include "OpenGL/SlateOpenGLTextures.h"
#include "OpenGL/SlateOpenGLRenderer.h"

/** Official OpenGL definitions */
#ifndef GL_HALF_FLOAT
#define GL_HALF_FLOAT 0x140B
#endif

#define BUFFER_OFFSET(i) ((uint8 *)NULL + (i))

/** Offset to apply to UVs to line up texels with pixels */
const float PixelCenterOffsetOpenGL = 0.0f;

/**
 * Returns the OpenGL primitive type to use when making draw calls                   
 */
static GLenum GetOpenGLPrimitiveType( ESlateDrawPrimitive SlateType )
{
	switch( SlateType )
	{
	case ESlateDrawPrimitive::LineList:
		return GL_LINES;
	case ESlateDrawPrimitive::TriangleList:
	default:
		return GL_TRIANGLES;
	}

};


FSlateOpenGLRenderingPolicy::FSlateOpenGLRenderingPolicy( TSharedRef<FSlateFontServices> InSlateFontServices, TSharedRef<FSlateOpenGLTextureManager> InTextureManager )
	: FSlateRenderingPolicy( InSlateFontServices, PixelCenterOffsetOpenGL )
	, VertexBuffer( sizeof(FSlateVertex) )
	, WhiteTexture( NULL )
	, TextureManager( InTextureManager )
	, bIsInitialized( false )
{
}

FSlateOpenGLRenderingPolicy::~FSlateOpenGLRenderingPolicy()
{
	ReleaseResources();
}

/** 
 * Initializes resources if needed
 */
void FSlateOpenGLRenderingPolicy::ConditionalInitializeResources()
{
	if( !bIsInitialized )
	{
		// Create shaders
		VertexShader.Create( FString::Printf( TEXT("%sShaders/StandaloneRenderer/OpenGL/SlateVertexShader.glsl"), *FPaths::EngineDir() ) );
		PixelShader.Create( FString::Printf( TEXT("%sShaders/StandaloneRenderer/OpenGL/SlateElementPixelShader.glsl"), *FPaths::EngineDir() ) );
		
		// Link shader programs
		ElementProgram.CreateProgram( VertexShader, PixelShader );

		// Create a default texture.
		check( WhiteTexture == NULL );
		WhiteTexture = TextureManager->CreateColorTexture( TEXT("DefaultWhite"), FColor::White )->GetSlateResource();

		bIsInitialized = true;
	}
}

/** 
 * Releases rendering resources
 */
void FSlateOpenGLRenderingPolicy::ReleaseResources()
{
	VertexBuffer.DestroyBuffer();
	IndexBuffer.DestroyBuffer();
}

/**
 * Updates vertex and index buffers used in drawing
 *
 * @param InVertices	The vertices to copy to the vertex buffer
 * @param InIndices		The indices to copy to the index buffer
 */
void FSlateOpenGLRenderingPolicy::BuildRenderingBuffers(FSlateBatchData& InBatchData)
{
	if( InBatchData.GetRenderBatches().Num() > 0 )
	{
		const FSlateVertexArray& FinalVertexData = InBatchData.GetFinalVertexData();
		const FSlateIndexArray& FinalIndexData = InBatchData.GetFinalIndexData();

		if( FinalVertexData.Num() > 0 )
		{
			const uint32 NumVertices = FinalVertexData.Num();
	
			// resize if needed
			if( NumVertices*sizeof(FSlateVertex) > VertexBuffer.GetBufferSize() )
			{
				uint32 NumBytesNeeded = NumVertices*sizeof(FSlateVertex);
				// increase by a static size.
				// @todo make this better
				VertexBuffer.ResizeBuffer( NumBytesNeeded + 200*sizeof(FSlateVertex) );
			}
		}

		if( FinalIndexData.Num() > 0 )
		{
			const uint32 NumIndices = FinalIndexData.Num();

			// resize if needed
			if( NumIndices > IndexBuffer.GetMaxNumIndices() )
			{
				// increase by a static size.
				// @todo make this better
				IndexBuffer.ResizeBuffer( NumIndices + 100 );
			}
		}

		uint8* VerticesPtr = (uint8*)VertexBuffer.Lock(0);
		uint8* IndicesPtr = (uint8*)IndexBuffer.Lock(0);

		//Early out if we have an invalid buffer (might have lost context and now have invalid buffers)
		if ((nullptr != VerticesPtr) && (nullptr != IndicesPtr))
		{
			FMemory::Memcpy(VerticesPtr, FinalVertexData.GetData(), FinalVertexData.Num() * sizeof(FSlateVertex));
			FMemory::Memcpy(IndicesPtr, FinalIndexData.GetData(), FinalIndexData.Num() * sizeof(SlateIndex));

		}

		if (nullptr != VerticesPtr)
		{
			VertexBuffer.Unlock();
		}

		if (nullptr != IndicesPtr)
		{
			IndexBuffer.Unlock();
		}
	}
}

void FSlateOpenGLRenderingPolicy::DrawElements( const FMatrix& ViewProjectionMatrix, FVector2D ViewportSize, TArrayView<const FSlateRenderBatch> RenderBatches)
{
	// Bind the vertex buffer.  Each element uses the same buffer
	VertexBuffer.Bind();

	// Bind the shader program.  Each element uses the same shader program
	ElementProgram.BindProgram();

	// Set the view projection matrix for the current viewport.
	ElementProgram.SetViewProjectionMatrix( ViewProjectionMatrix );

	// OpenGL state toggles
	glDisable(GL_DEPTH_TEST);
	glDisable(GL_CULL_FACE);
#if !PLATFORM_USES_GLES && !PLATFORM_LINUX
	glEnable(GL_TEXTURE_2D);
#endif

	// Set up alpha testing
#if !PLATFORM_USES_GLES && !PLATFORM_LINUX
	glEnable(GL_ALPHA_TEST);
	glAlphaFunc( GL_GREATER, 0.0f );
#endif
	
	// Set up blending
	glBlendFunc( GL_SRC_ALPHA, GL_ONE_MINUS_SRC_ALPHA );
	glBlendEquation( GL_FUNC_ADD );

	// Setup stencil
	glStencilMask(0xFF);
	glStencilFunc(GL_GREATER, 0, 0xFF);
	glStencilOp(GL_KEEP, GL_INCR, GL_INCR);

	const FSlateClippingState* LastClippingState = nullptr;

	int32 NextRenderBatchIndex = 0;
	while (NextRenderBatchIndex != INDEX_NONE)
	{
		const FSlateRenderBatch& RenderBatch = RenderBatches[NextRenderBatchIndex];

		NextRenderBatchIndex = RenderBatch.NextBatchIndex;

		const FSlateShaderResource* ShaderResource = RenderBatch.GetShaderResource();

		const ESlateBatchDrawFlag DrawFlags = RenderBatch.GetDrawFlags();

		if( EnumHasAllFlags(DrawFlags, ESlateBatchDrawFlag::NoBlending) )
		{
			glDisable( GL_BLEND );
		}
		else
		{
			glEnable( GL_BLEND );
		}

#if !PLATFORM_USES_GLES
		if( EnumHasAllFlags(DrawFlags, ESlateBatchDrawFlag::Wireframe) )
		{
			glPolygonMode( GL_FRONT_AND_BACK, GL_LINE );
			glDisable(GL_BLEND);
		}
		else
		{
			glPolygonMode( GL_FRONT_AND_BACK, GL_FILL );
		}
#endif

		if (EnumHasAllFlags(DrawFlags, ESlateBatchDrawFlag::NoGamma))
		{
			ElementProgram.SetGammaValues(FVector2f(1.0f, 1.0f));
		}
		else
		{
			ElementProgram.SetGammaValues(FVector2f(1, 1 / 2.2f));
		}

		ElementProgram.SetShaderType( static_cast<uint8>(RenderBatch.GetShaderType()) );
		ElementProgram.SetShaderParams(RenderBatch.GetShaderParams());
		ElementProgram.SetDrawEffects( RenderBatch.GetDrawEffects() );

		// Disable stenciling and depth testing by default
		glDisable(GL_STENCIL_TEST);
		
		if( RenderBatch.GetShaderType() == ESlateShader::LineSegment )
		{
			// Test that the pixels in the line segment have only been drawn once
			glEnable(GL_STENCIL_TEST);
		}
		else if(ShaderResource)
		{
			uint32 RepeatU = EnumHasAllFlags(DrawFlags, ESlateBatchDrawFlag::TileU) ? GL_REPEAT : GL_CLAMP_TO_EDGE;
			uint32 RepeatV = EnumHasAllFlags(DrawFlags, ESlateBatchDrawFlag::TileV) ? GL_REPEAT : GL_CLAMP_TO_EDGE;

#if PLATFORM_USES_GLES
			if(!FMath::IsPowerOfTwo(Texture->GetWidth()) || !FMath::IsPowerOfTwo(Texture->GetHeight()))
			{
				RepeatU = GL_CLAMP_TO_EDGE;
				RepeatV = GL_CLAMP_TO_EDGE;
			}
#endif

			ElementProgram.SetTexture( (FSlateOpenGLTexture*)ShaderResource, RepeatU, RepeatV );
		}
		else
		{
			ElementProgram.SetTexture( (FSlateOpenGLTexture*)WhiteTexture, GL_CLAMP_TO_EDGE, GL_CLAMP_TO_EDGE );
		}

		check( RenderBatch.GetNumIndices() > 0 );

		const uint32 IndexCount = RenderBatch.GetNumIndices();
		check(IndexCount > 0);

		uint32 Offset = 0;
		// The offset into the vertex buffer where this batches vertices are located
		uint32 BaseVertexIndex = RenderBatch.GetVertexOffset();
		// The starting index in the index buffer for this element batch
		uint32 StartIndex = RenderBatch.GetIndexOffset() * sizeof(SlateIndex);


		// Size of each vertex
		const uint32 Stride = sizeof(FSlateVertex);

		// Set up offsets into the vertex buffer for each vertex
		glEnableVertexAttribArray(0);
		Offset = STRUCT_OFFSET(FSlateVertex, TexCoords);
		glVertexAttribPointer(0, 4, GL_FLOAT, GL_FALSE, Stride, BUFFER_OFFSET(Stride*BaseVertexIndex+Offset));

		glEnableVertexAttribArray(1);
		Offset = STRUCT_OFFSET(FSlateVertex, Position);
		glVertexAttribPointer(1, 2, GL_FLOAT, GL_FALSE, Stride, BUFFER_OFFSET(Stride*BaseVertexIndex+Offset));

		glEnableVertexAttribArray(2);
		Offset = STRUCT_OFFSET(FSlateVertex, Color);
		glVertexAttribPointer(2, 4, GL_UNSIGNED_BYTE, GL_TRUE, Stride, BUFFER_OFFSET(Stride*BaseVertexIndex+Offset));

		glEnableVertexAttribArray(3);
		Offset = STRUCT_OFFSET(FSlateVertex, SecondaryColor);
		glVertexAttribPointer(3, 4, GL_UNSIGNED_BYTE, GL_TRUE, Stride, BUFFER_OFFSET(Stride*BaseVertexIndex+Offset));
		
		// Bind the index buffer so glDrawRangeElements knows which one to use
		IndexBuffer.Bind();


		if (RenderBatch.GetClippingState() != LastClippingState)
		{
			LastClippingState = RenderBatch.GetClippingState();

			if (LastClippingState)
			{
				const FSlateClippingState& ClipState = *LastClippingState;
				if (ClipState.ScissorRect.IsSet())
				{
					const FSlateClippingZone& ScissorRect = ClipState.ScissorRect.GetValue();

					glEnable(GL_SCISSOR_TEST);
					
					const float ScissorWidth = FVector2f::Distance(ScissorRect.TopLeft, ScissorRect.TopRight);
					const float ScissorHeight = FVector2f::Distance(ScissorRect.TopLeft, ScissorRect.BottomLeft);
					glScissor(ScissorRect.TopLeft.X, ViewportSize.Y - ScissorRect.BottomLeft.Y, ScissorWidth, ScissorHeight);
				}
				else
				{
					// We don't support stencil clipping on the d3d rendering policy.
					glDisable(GL_SCISSOR_TEST);
				}
			}
			else
			{
				glDisable(GL_SCISSOR_TEST);
			}
		}
		
#if SLATE_USE_32BIT_INDICES
#define GL_INDEX_FORMAT GL_UNSIGNED_INT
#else
#define GL_INDEX_FORMAT GL_UNSIGNED_SHORT
#endif

#if PLATFORM_USES_GLES
		glDrawElements(GetOpenGLPrimitiveType(RenderBatch.GetDrawPrimitiveType()), RenderBatch.GetNumIndices(), GL_INDEX_FORMAT, (void*)StartIndex);
#else
		// Draw all elements in batch
		glDrawRangeElements( GetOpenGLPrimitiveType(RenderBatch.GetDrawPrimitiveType()), 0, RenderBatch.GetNumVertices(), RenderBatch.GetNumIndices(), GL_INDEX_FORMAT, (void*)(PTRINT)StartIndex );
#endif
		CHECK_GL_ERRORS;
		
	}

	// Disable active textures and shaders
	glDisable(GL_SCISSOR_TEST);
	glActiveTexture( GL_TEXTURE0 );
	glBindTexture( GL_TEXTURE_2D, 0 );
	glUseProgram(0);

}

TSharedRef<FSlateShaderResourceManager> FSlateOpenGLRenderingPolicy::GetResourceManager() const
{
	return TextureManager.ToSharedRef();
}
