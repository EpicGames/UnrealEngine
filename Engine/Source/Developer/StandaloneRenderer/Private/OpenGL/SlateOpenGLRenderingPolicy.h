// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Rendering/ShaderResourceManager.h"
#include "Rendering/DrawElements.h"
#include "Rendering/SlateRenderer.h"
#include "OpenGL/SlateOpenGLTextureManager.h"
#include "Rendering/RenderingPolicy.h"
#include "OpenGL/SlateOpenGLShaders.h"
#include "OpenGL/SlateOpenGLIndexBuffer.h"
#include "OpenGL/SlateOpenGLVertexBuffer.h"
#include "Layout/Clipping.h"

class FSlateOpenGLTexture;

class FSlateOpenGLRenderingPolicy : public FSlateRenderingPolicy
{
public:
	FSlateOpenGLRenderingPolicy( TSharedRef<FSlateFontServices> InSlateFontServices, TSharedRef<FSlateOpenGLTextureManager> InTextureManager );
	~FSlateOpenGLRenderingPolicy();

	/**
	 * Updates vertex and index buffers used in drawing
	 *
	 * @param BatchData	The batch data that contains rendering data we need to upload to the buffers
	 */
	void BuildRenderingBuffers(FSlateBatchData& BatchData);

	/**
	 * Draws Slate elements
	 *
	 * @param ViewProjectionMatrix	The view projection matrix to pass to the vertex shader
	 * @param RenderBatches			A list of batches that should be rendered.
	 */
	void DrawElements( const FMatrix& ViewProjectionMatrix, FVector2D ViewportSize, TArrayView<const FSlateRenderBatch> RenderBatches);

	virtual TSharedRef<FSlateShaderResourceManager> GetResourceManager() const override;

	virtual bool IsVertexColorInLinearSpace() const override { return false; }

	/** 
	 * Initializes resources if needed
	 */
	void ConditionalInitializeResources();

	/** 
	 * Releases rendering resources
	 */
	void ReleaseResources();
private:
	/** Vertex shader used for all elements */
	FSlateOpenGLVS VertexShader;
	/** Pixel shader used for all elements */
	FSlateOpenGLPS	PixelShader;
	/** Shader program for all elements */
	FSlateOpenGLElementProgram ElementProgram;
	/** Vertex buffer containing all the vertices of every element */
	FSlateOpenGLVertexBuffer VertexBuffer;
	/** Index buffer for accessing vertices of elements */
	FSlateOpenGLIndexBuffer IndexBuffer;
	/** A default white texture to use if no other texture can be found */
	FSlateShaderResource* WhiteTexture;
	/** Texture manager for accessing OpenGL textures */
	TSharedPtr<FSlateOpenGLTextureManager> TextureManager;
	/** True if the rendering policy has been initialized */
	bool bIsInitialized;
};

