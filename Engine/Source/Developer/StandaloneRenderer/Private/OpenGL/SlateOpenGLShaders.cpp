// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenGL/SlateOpenGLShaders.h"
#include "Misc/FileHelper.h"
#include "OpenGL/SlateOpenGLExtensions.h"

#include "OpenGL/SlateOpenGLRenderer.h"

#define USE_709 0

/**
 * Returns the current program log for a GLSL program                   
 */
static FString GetGLSLProgramLog( GLuint Program )
{
	GLint Len;
	FString ProgramLog;
	glGetProgramiv( Program, GL_INFO_LOG_LENGTH, &Len );

	if( Len > 0 )
	{
		GLchar* Log = new GLchar[Len];

		GLsizei ActualLen;
		glGetProgramInfoLog( Program, Len, &ActualLen, Log );

		ProgramLog = ANSI_TO_TCHAR( Log );

		delete[] Log;
	}

	return ProgramLog;
}

/**
 * Returns the current shader log for a GLSL shader                   
 */
static FString GetGLSLShaderLog( GLuint Shader )
{
	GLint Len;
	FString ShaderLog;

	glGetShaderiv(Shader, GL_INFO_LOG_LENGTH, &Len);

	if(Len > 0)
	{
		GLsizei ActualLen;
		GLchar *Log = new GLchar[Len];

		glGetShaderInfoLog(Shader, Len, &ActualLen, Log);
	
		ShaderLog = ANSI_TO_TCHAR( Log );

		delete[] Log;
	}

	return ShaderLog;
}

FSlateOpenGLShader::FSlateOpenGLShader()
	: ShaderID(0)
{

}

FSlateOpenGLShader::~FSlateOpenGLShader()
{
	if( ShaderID > 0 )
	{
		glDeleteShader( ShaderID );
	}
}

/**
 * Creates an compiles a GLSL shader
 *
 * @param Filename		A path to the GLSL file
 * @param ShaderType	The type of shader being compiled
 */

void FSlateOpenGLShader::CompileShader( const FString& Filename, GLenum ShaderType )
{
	// Create a new shader ID.
	ShaderID = glCreateShader( ShaderType );
	GLint CompileStatus = GL_FALSE;

	check( ShaderID );

	// Load the file to a string
	FString Source;
	bool bFileFound = FFileHelper::LoadFileToString( Source, *Filename );
	check(bFileFound);
	
	FString Header;
	
	// pass the #define along to the shader
#if PLATFORM_USES_GLES
	Header.Append("#define PLATFORM_USES_GLES 1\n");
#elif PLATFORM_LINUX
	#if LINUX_USE_OPENGL_3_2
	Header.Append("#version 150\n#define PLATFORM_USES_GLES 0\n");
	#else
	Header.Append("#version 120\n#define PLATFORM_USES_GLES 0\n");
	#endif // LINUX_USE_OPENGL_3_2
#else
	Header.Append("#version 120\n#define PLATFORM_USES_GLES 0\n");
#endif
	
#if PLATFORM_LINUX
	Header.Append("#define PLATFORM_LINUX 1\n");
#else
	Header.Append("#define PLATFORM_LINUX 0\n");
#endif
	
#if PLATFORM_MAC
	Header.Append("#define PLATFORM_MAC 1\n");
#else
	Header.Append("#define PLATFORM_MAC 0\n");
#endif
	
#if USE_709
	Header.Append("#define USE_709 1\n");
#else
	Header.Append("#define USE_709 0\n");
#endif
	
	// Allocate a buffer big enough to store the string in ascii format
	ANSICHAR* Chars[2] = {0};
	
	Chars[0] = new ANSICHAR[Header.Len()+1];
	FCStringAnsi::Strncpy(Chars[0], TCHAR_TO_ANSI(*Header), Header.Len() + 1);
	
	Chars[1] = new ANSICHAR[Source.Len()+1];
	FCStringAnsi::Strncpy(Chars[1], TCHAR_TO_ANSI(*Source), Source.Len() + 1);

	// give opengl the source code for the shader
	glShaderSource( ShaderID, 2, (const ANSICHAR**)Chars, NULL );
	delete[] Chars[0];
	delete[] Chars[1];

	// Compile the shader and check for success
	glCompileShader( ShaderID );

	glGetShaderiv( ShaderID, GL_COMPILE_STATUS, &CompileStatus );
	if( CompileStatus == GL_FALSE )
	{
		// The shader did not compile.  Display why it failed.
		FString Log = GetGLSLShaderLog( ShaderID );

		checkf(false, TEXT("Failed to compile shader: %s\n%s"), *Filename, *Log );

		// Delete the shader since it failed.
		glDeleteShader( ShaderID );
		ShaderID = 0;
	}
}

FSlateOpenGLVS::FSlateOpenGLVS()
{

}

FSlateOpenGLVS::~FSlateOpenGLVS()
{
	if( ShaderID > 0 )
	{
		glDeleteShader( ShaderID );
	}
}


void FSlateOpenGLVS::Create( const FString& Filename )
{
	check( ShaderID==0 );
	// Compile the vertex shader
	CompileShader( Filename, GL_VERTEX_SHADER );
}

FSlateOpenGLPS::FSlateOpenGLPS()	
{

}

FSlateOpenGLPS::~FSlateOpenGLPS()
{
	if( ShaderID > 0 )
	{
		glDeleteShader( ShaderID );
	}
}

void FSlateOpenGLPS::Create( const FString& Filename )
{
	check( ShaderID==0 );
	// Compile the pixel shader
	CompileShader( Filename, GL_FRAGMENT_SHADER );
}

FSlateOpenGLShaderProgram::FSlateOpenGLShaderProgram()
	: ProgramID(0)
{

}

FSlateOpenGLShaderProgram::~FSlateOpenGLShaderProgram()
{
	if( ProgramID > 0 )
	{
		glDeleteProgram(ProgramID);
		ProgramID = 0;
	}
}

void FSlateOpenGLShaderProgram::BindProgram()
{
	glUseProgram( ProgramID );
	CHECK_GL_ERRORS;
}

/**
 * Links a vertex shader and pixel shader into a program for use in rendering     
 * 
 * @param VertexShader	The vertex shader to link
 * @param PixelShader	The pixel shader to link
 */
void FSlateOpenGLShaderProgram::LinkShaders( const FSlateOpenGLVS& VertexShader, const FSlateOpenGLPS& PixelShader )
{
	// Should not link twice
	check( ProgramID == 0 );

	GLuint VertexShaderID = VertexShader.GetShaderID();
	GLuint PixelShaderID = PixelShader.GetShaderID();

	// Make sure the shaders have been created
	check( VertexShaderID && PixelShaderID );

	// Create a new program id and attach the shaders
	ProgramID = glCreateProgram();
	glAttachShader( ProgramID, VertexShaderID );
	glAttachShader( ProgramID, PixelShaderID );
	CHECK_GL_ERRORS;

	// Set up attribute locations for per vertex data
	glBindAttribLocation(ProgramID, 0, "InTexCoords");
	glBindAttribLocation(ProgramID, 1, "InPosition");
	glBindAttribLocation(ProgramID, 2, "InColor");
	glBindAttribLocation(ProgramID, 3, "InSecondaryColor");

	// Link the shaders
	glLinkProgram( ProgramID );
	CHECK_GL_ERRORS;

	// Check to see if linking succeeded
	GLint LinkStatus;
	glGetProgramiv( ProgramID, GL_LINK_STATUS, &LinkStatus );
	if( LinkStatus == GL_FALSE )
	{
		// Linking failed, display why.
		FString Log = GetGLSLProgramLog( ProgramID );

		checkf(false, TEXT("Failed to link GLSL program: %s"), *Log );
	}

	CHECK_GL_ERRORS;
}

void FSlateOpenGLElementProgram::CreateProgram( const FSlateOpenGLVS& VertexShader, const FSlateOpenGLPS& PixelShader )
{
	// Link the vertex and pixel shader for this program
	LinkShaders( VertexShader, PixelShader );

	// Set up uniform parameters
	ViewProjectionMatrixParam = glGetUniformLocation( ProgramID, "ViewProjectionMatrix" );
	VertexShaderParam = glGetUniformLocation( ProgramID, "VertexShaderParams" );
	TextureParam = glGetUniformLocation( ProgramID, "ElementTexture" );
	EffectsDisabledParam = glGetUniformLocation( ProgramID, "EffectsDisabled" );
	IgnoreTextureAlphaParam = glGetUniformLocation( ProgramID, "IgnoreTextureAlpha" );
#if PLATFORM_MAC
	TextureRectParam = glGetUniformLocation( ProgramID, "ElementRectTexture" );
	UseTextureRectangle = glGetUniformLocation( ProgramID, "UseTextureRectangle" );
	SizeParam = glGetUniformLocation( ProgramID, "Size" );
#endif
	ShaderTypeParam = glGetUniformLocation( ProgramID, "ShaderType" );
	ShaderParamsParam = glGetUniformLocation( ProgramID, "ShaderParams" );
	ShaderParams2Param = glGetUniformLocation(ProgramID, "ShaderParams2");
	GammaValuesParam = glGetUniformLocation(ProgramID, "GammaValues");

	CHECK_GL_ERRORS;
}


void FSlateOpenGLElementProgram::SetTexture( FSlateOpenGLTexture *Texture, uint32 AddressU, uint32 AddressV )
{
	GLint TargetTextureType = Texture->GetTextureTargetType();
#if PLATFORM_MAC
	// Set the texture parameter to use
	if (TargetTextureType == GL_TEXTURE_RECTANGLE_ARB)
	{
		glUniform1i( UseTextureRectangle, 1 );
		glUniform2f( SizeParam, (GLfloat)Texture->GetWidth(), (GLfloat)Texture->GetHeight() );
		// Use the 2nd texture unit for ARB type textures
		glActiveTexture( GL_TEXTURE1 );
	}
	else
	{
		glUniform1i( UseTextureRectangle, 0 );
		// Set the first texture as active
		glActiveTexture( GL_TEXTURE0 );
	}

	glUniform1i(TextureRectParam, 1);
#else
	// Set the first texture as active
	glActiveTexture(GL_TEXTURE0);
#endif
	glUniform1i( TextureParam, 0 );
	CHECK_GL_ERRORS;

	glEnable(TargetTextureType);
	// bind the texture
	glBindTexture(TargetTextureType, Texture->GetTypedResource() );

	glTexParameteri(TargetTextureType, GL_TEXTURE_WRAP_S, AddressU);
	glTexParameteri(TargetTextureType, GL_TEXTURE_WRAP_T, AddressV);

	CHECK_GL_ERRORS;
}

void FSlateOpenGLElementProgram::SetViewProjectionMatrix( const FMatrix& InVP )
{
	FMatrix44f InVPFlt(InVP);	// LWC_TODO: Precision loss?
	const GLfloat* Param = &InVPFlt.M[0][0];
	glUniformMatrix4fv( ViewProjectionMatrixParam, 1, GL_FALSE, Param );
	CHECK_GL_ERRORS;
}

void FSlateOpenGLElementProgram::SetVertexShaderParams( const FVector4f& ShaderParams )
{
	glUniform4f( VertexShaderParam, ShaderParams.X, ShaderParams.Y, ShaderParams.Z, ShaderParams.W );
	CHECK_GL_ERRORS;
}

void FSlateOpenGLElementProgram::SetDrawEffects(ESlateDrawEffect InDrawEffects )
{
	glUniform1i( EffectsDisabledParam, EnumHasAllFlags(InDrawEffects, ESlateDrawEffect::DisabledEffect) ? 1 : 0 );
	glUniform1i( IgnoreTextureAlphaParam, EnumHasAllFlags(InDrawEffects, ESlateDrawEffect::IgnoreTextureAlpha) ? 1 : 0 );
	CHECK_GL_ERRORS;
}


void FSlateOpenGLElementProgram::SetShaderType( uint32 InShaderType )
{
	glUniform1i( ShaderTypeParam, InShaderType );
	CHECK_GL_ERRORS;
}

void FSlateOpenGLElementProgram::SetShaderParams(const FShaderParams& InShaderParams)
{
	const GLfloat* Params = (GLfloat*)&InShaderParams.PixelParams;
	glUniform4fv(ShaderParamsParam, 1, Params);

	const GLfloat* Params2 = (GLfloat*)&InShaderParams.PixelParams2;
	glUniform4fv(ShaderParams2Param, 1, Params2);

	CHECK_GL_ERRORS;
}

void FSlateOpenGLElementProgram::SetGammaValues(const FVector2f& InGammaValues)
{
	glUniform2f(GammaValuesParam, InGammaValues.X, InGammaValues.Y);
	CHECK_GL_ERRORS;
}
