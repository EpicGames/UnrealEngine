// Copyright Epic Games, Inc. All Rights Reserved.

/**
*/

#pragma once

#include "CoreMinimal.h"
#include "StandaloneRendererPlatformHeaders.h"
#include "Rendering/RenderingCommon.h"
#include "SlateOpenGLTextures.h"

/**
 * Base class for all OpenGL shaders                   
 */
class FSlateOpenGLShader
{
public:
	FSlateOpenGLShader();
	virtual ~FSlateOpenGLShader();
	virtual void Create( const FString& Filename ) = 0;
	GLuint GetShaderID() const { return ShaderID; }
protected:
	void CompileShader( const FString& Filename, GLenum ShaderType );
protected:
	GLuint ShaderID;
};
/**
 * Represents an OpenGL vertex shader.  Do not inherit from this for specific shaders.  Inherit from FSlateOpenGLProgram
 */
class FSlateOpenGLVS : public FSlateOpenGLShader
{
public:
	FSlateOpenGLVS();
	~FSlateOpenGLVS();
	virtual void Create( const FString& Filename );
};

/**
 * Represents an OpenGL vertex shader.  Do not inherit from this for specific shaders.  Inherit from FSlateOpenGLProgram
 */
class FSlateOpenGLPS : public FSlateOpenGLShader
{
public:
	FSlateOpenGLPS();
	~FSlateOpenGLPS();
	virtual void Create( const FString& Filename );
};


class FSlateOpenGLShaderProgram
{
public:
	FSlateOpenGLShaderProgram();
	virtual ~FSlateOpenGLShaderProgram();
	virtual void CreateProgram( const FSlateOpenGLVS& VertexShader, const FSlateOpenGLPS& PixelShader ) = 0;
	void BindProgram();
protected:
	void LinkShaders( const FSlateOpenGLVS& VertexShader, const FSlateOpenGLPS& PixelShader );
protected:
	GLuint ProgramID;
};

class FSlateOpenGLElementProgram : public FSlateOpenGLShaderProgram
{
public:
	virtual void CreateProgram( const FSlateOpenGLVS& VertexShader, const FSlateOpenGLPS& PixelShader );
	void SetViewProjectionMatrix( const FMatrix& InVP ); 
	void SetVertexShaderParams( const FVector4f& ShaderParams );
	void SetTexture( FSlateOpenGLTexture *Texture, uint32 AddressU, uint32 AddressV  );
	void SetDrawEffects(ESlateDrawEffect InDrawEffects );
	void SetShaderType( uint32 InShaderType );
	void SetShaderParams(const FShaderParams& InShaderParams);
	void SetGammaValues(const FVector2f& InGammaValues);
private:
	GLint ViewProjectionMatrixParam;
	GLint VertexShaderParam;
	GLint TextureParam;
	GLint EffectsDisabledParam;
	GLint IgnoreTextureAlphaParam;
#if PLATFORM_MAC
	GLint TextureRectParam;
	GLint SizeParam;
	GLint UseTextureRectangle;
#endif
	GLint ShaderTypeParam;
	GLint ShaderParamsParam;
	GLint ShaderParams2Param;
	GLint GammaValuesParam;
};
