// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenGL/SlateOpenGLTextureManager.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "StandaloneRendererPlatformHeaders.h"
#include "OpenGL/SlateOpenGLTextures.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Rendering/SlateVectorGraphicsCache.h"


DEFINE_LOG_CATEGORY_STATIC(LogSlateOpenGL, Log, All);

class FSlateOpenGLTextureAtlasFactory : public ISlateTextureAtlasFactory
{
public:
	virtual TUniquePtr<FSlateTextureAtlas> CreateTextureAtlas(int32 AtlasSize, int32 AtlasStride, ESlateTextureAtlasPaddingStyle PaddingStyle, bool bUpdatesAfterInitialization) const
	{
		return CreateTextureAtlasInternal(AtlasSize, AtlasStride, PaddingStyle, bUpdatesAfterInitialization);
	}

	virtual TUniquePtr<FSlateShaderResource> CreateNonAtlasedTexture(const uint32 InWidth, const uint32 InHeight, const TArray<uint8>& InRawData) const
	{
		// Create a representation of the texture for rendering
		TUniquePtr<FSlateOpenGLTexture> NewTexture = MakeUnique<FSlateOpenGLTexture>(InWidth, InHeight);
#if !PLATFORM_USES_GLES
		NewTexture->Init(GL_SRGB8_ALPHA8, InRawData);
#else
		NewTexture->Init(GL_SRGB8_ALPHA8_EXT, InRawData);
#endif

		return NewTexture;
	}

	virtual void ReleaseTextureAtlases(const TArray<TUniquePtr<FSlateTextureAtlas>>& InTextureAtlases, const TArray<TUniquePtr<FSlateShaderResource>>& InNonAtlasedTextures, const bool bWaitForRelease) const
	{
		// nothing to do
	}

	static TUniquePtr<FSlateTextureAtlasOpenGL> CreateTextureAtlasInternal(int32 AtlasSize, int32 AtlasStride, ESlateTextureAtlasPaddingStyle PaddingStyle, bool bUpdatesAfterInitialization)
	{
		return MakeUnique<FSlateTextureAtlasOpenGL>(AtlasSize, AtlasSize, AtlasStride, PaddingStyle);
	}
};


FSlateOpenGLTextureManager::FDynamicTextureResource::FDynamicTextureResource(FSlateOpenGLTexture* InOpenGLTexture)
	: Proxy(new FSlateShaderResourceProxy)
	, OpenGLTexture(InOpenGLTexture)
{
}

FSlateOpenGLTextureManager::FDynamicTextureResource::~FDynamicTextureResource()
{
	if (Proxy)
	{
		delete Proxy;
	}

	if (OpenGLTexture)
	{
		delete OpenGLTexture;
	}
}


FSlateOpenGLTextureManager::FSlateOpenGLTextureManager()
{
	VectorGraphicsCache = MakeUnique<FSlateVectorGraphicsCache>(MakeShared<FSlateOpenGLTextureAtlasFactory>());
}

FSlateOpenGLTextureManager::~FSlateOpenGLTextureManager()
{
}

void FSlateOpenGLTextureManager::LoadUsedTextures()
{
	TArray< const FSlateBrush* > Resources;
	FSlateStyleRegistry::GetAllResources(Resources);

	CreateTextures(Resources);
}

void FSlateOpenGLTextureManager::LoadStyleResources(const ISlateStyle& Style)
{
	TArray< const FSlateBrush* > Resources;
	Style.GetResources(Resources);

	CreateTextures(Resources);
}

void FSlateOpenGLTextureManager::CreateTextures(const TArray< const FSlateBrush* >& Resources)
{
	TMap<FName, FNewTextureInfo> TextureInfoMap;

	for (int32 ResourceIndex = 0; ResourceIndex < Resources.Num(); ++ResourceIndex)
	{
		const FSlateBrush& Brush = *Resources[ResourceIndex];

		const FName TextureName = Brush.GetResourceName();

		if (Brush.GetImageType() != ESlateBrushImageType::Vector && TextureName != NAME_None && !ResourceMap.Contains(TextureName))
		{
			// Find the texture or add it if it doesnt exist (only load the texture once)
			FNewTextureInfo& Info = TextureInfoMap.FindOrAdd(TextureName);
			Info.bSrgb = (Brush.ImageType != ESlateBrushImageType::Linear);

			// Only atlas the texture if none of the brushes that use it tile it
			Info.bShouldAtlas &= (Brush.Tiling == ESlateBrushTileType::NoTile && Info.bSrgb);

			if (!Info.TextureData.IsValid())
			{
				uint32 Width = 0;
				uint32 Height = 0;
				TArray<uint8> RawData;
				bool bSucceeded = LoadTexture(Brush, Width, Height, RawData);
				const uint32 Stride = 4; //RGBA
				Info.TextureData = MakeShareable(new FSlateTextureData(Width, Height, Stride, RawData));

				const bool bTooLargeForAtlas = (Width >= 256 || Height >= 256);

				Info.bShouldAtlas &= !bTooLargeForAtlas;

				if (!bSucceeded)
				{
					TextureInfoMap.Remove(TextureName);
				}

			}
		}
	}

	TextureInfoMap.ValueSort(FCompareFNewTextureInfoByTextureSize());

	for (TMap<FName, FNewTextureInfo>::TConstIterator It(TextureInfoMap); It; ++It)
	{
		const FNewTextureInfo& Info = It.Value();
		FName TextureName = It.Key();
		FString NameStr = TextureName.ToString();

		FSlateShaderResourceProxy* NewTexture = GenerateTextureResource(Info, TextureName);

		ResourceMap.Add(TextureName, NewTexture);
	}

}

/**
 * Creates a texture from a file on disk
 *
 * @param TextureName	The name of the texture to load
 */
bool FSlateOpenGLTextureManager::LoadTexture(const FSlateBrush& InBrush, uint32& OutWidth, uint32& OutHeight, TArray<uint8>& OutDecodedImage)
{
	FName TextureName = InBrush.GetResourceName();

	bool bSucceeded = false;

	// Get the path to the resource
	FString ResourcePath = GetResourcePath(InBrush);

	// Load the resource into an array
	TArray<uint8> Buffer;
	if (FFileHelper::LoadFileToArray(Buffer, *ResourcePath, FILEREAD_Silent))
	{
		IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));

		//Try and determine format, if that fails assume PNG
		EImageFormat ImageFormat = ImageWrapperModule.DetectImageFormat(Buffer.GetData(), Buffer.Num());
		if (ImageFormat == EImageFormat::Invalid)
		{
			ImageFormat = EImageFormat::PNG;
		}
		TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(ImageFormat);

		if (ImageWrapper.IsValid() && ImageWrapper->SetCompressed(Buffer.GetData(), Buffer.Num()))
		{
			// Determine the block size.  This is bytes per pixel
			const uint8 BlockSize = 4;

			OutWidth = ImageWrapper->GetWidth();
			OutHeight = ImageWrapper->GetHeight();

			// Decode the png and get the data in raw rgb
			if (ImageWrapper->GetRaw(ERGBFormat::BGRA, 8, OutDecodedImage))
			{
				bSucceeded = true;
			}
			else
			{
				UE_LOGF(LogSlateOpenGL, Log, "Couldn't convert to raw data. [%ls] '%ls'", *InBrush.GetResourceName().ToString(), *ResourcePath);
			}
		}
		else
		{
			UE_LOGF(LogSlateOpenGL, Log, "Only pngs are supported in Slate. [%ls] '%ls'", *InBrush.GetResourceName().ToString(), *ResourcePath);
		}
	}
	else
	{
		UE_CLOGF(!IsRunningCommandlet(), LogSlateOpenGL, Log, "Could not load file for Slate resource: [%ls] '%ls'", *InBrush.GetResourceName().ToString(), *ResourcePath);
	}


	return bSucceeded;
}

FSlateOpenGLTexture* FSlateOpenGLTextureManager::CreateColorTexture(const FName TextureName, FColor InColor)
{
	FNewTextureInfo Info;

	TArray<uint8> RawData;
	RawData.AddUninitialized(4);
	RawData[0] = InColor.R;
	RawData[1] = InColor.G;
	RawData[2] = InColor.B;
	RawData[3] = InColor.A;
	Info.bShouldAtlas = false;

	uint32 Width = 1;
	uint32 Height = 1;
	uint32 Stride = 4;
	Info.TextureData = MakeShareable(new FSlateTextureData(Width, Height, Stride, RawData));

	FSlateShaderResourceProxy* TextureProxy = GenerateTextureResource(Info, TextureName);

	// Cache the texture proxy for fast access later when we need the texture for rendering
	ResourceMap.Add(TextureName, TextureProxy);

	return (FSlateOpenGLTexture*)TextureProxy->Resource;
}

FSlateShaderResourceProxy* FSlateOpenGLTextureManager::GenerateTextureResource(const FNewTextureInfo& Info, FName TextureName)
{
	FSlateShaderResourceProxy* NewProxy = NULL;

	const uint32 Width = Info.TextureData->GetWidth();
	const uint32 Height = Info.TextureData->GetHeight();

	if (Info.bShouldAtlas)
	{
		const uint32 AtlasSize = 1024;
		// 4 bytes per pixel
		const uint32 AtlasStride = 4;
		// always use one pixel padding.
		const uint8 Padding = 1;
		const FAtlasedTextureSlot* NewSlot = nullptr;

		FSlateTextureAtlasOpenGL* Atlas = nullptr;
		// Get the last atlas and find a slot for the texture
		for (int32 AtlasIndex = 0; AtlasIndex < PrecachedTextureAtlases.Num(); ++AtlasIndex)
		{
			Atlas = PrecachedTextureAtlases[AtlasIndex].Get();
			NewSlot = Atlas->AddTexture(Width, Height, Info.TextureData->GetRawBytes());
			if (NewSlot)
			{
				break;
			}
		}

		// No new slot was found in any atlas so we have to make another one
		if (!NewSlot)
		{
			// A new slot in the atlas could not be found, make a new atlas and add the texture to it
			TUniquePtr<FSlateTextureAtlasOpenGL> NewAtlas = FSlateOpenGLTextureAtlasFactory::CreateTextureAtlasInternal(AtlasSize, AtlasStride, ESlateTextureAtlasPaddingStyle::DilateBorder, true);
			NewSlot = NewAtlas->AddTexture(Width, Height, Info.TextureData->GetRawBytes());

			Atlas = NewAtlas.Get();

			PrecachedTextureAtlases.Emplace(MoveTemp(NewAtlas));
		}

		check(Atlas && NewSlot);

#if WITH_ATLAS_DEBUGGING
		AtlasDebugData.Add(NewSlot, TextureName);
#endif
		// Create a proxy representing this texture in the atlas
		NewProxy = new FSlateShaderResourceProxy;
		NewProxy->Resource = Atlas->GetAtlasTexture();
		// Compute the sub-uvs for the location of this texture in the atlas, accounting for padding
		NewProxy->StartUV = FVector2f((float)(NewSlot->X + Padding) / Atlas->GetWidth(), (float)(NewSlot->Y + Padding) / Atlas->GetHeight());
		NewProxy->SizeUV = FVector2f((float)(NewSlot->Width - Padding * 2) / Atlas->GetWidth(), (float)(NewSlot->Height - Padding * 2) / Atlas->GetHeight());
		NewProxy->ActualSize = FIntPoint(Width, Height);
	}
	else
	{
		// Create a representation of the texture for rendering
		TUniquePtr<FSlateOpenGLTexture> NewTexture = MakeUnique<FSlateOpenGLTexture>(Width, Height);
#if !PLATFORM_USES_GLES
		NewTexture->Init(Info.bSrgb ? GL_SRGB8_ALPHA8 : GL_RGBA8, Info.TextureData->GetRawBytes());
#else
		NewTexture->Init(Info.bSrgb ? GL_SRGB8_ALPHA8_EXT : GL_RGBA8, Info.TextureData->GetRawBytes());
#endif

		NewProxy = new FSlateShaderResourceProxy;

		NewProxy->Resource = NewTexture.Get();
		NewProxy->StartUV = FVector2f(0.0f, 0.0f);
		NewProxy->SizeUV = FVector2f(1.0f, 1.0f);
		NewProxy->ActualSize = FIntPoint(Width, Height);

		NonAtlasedTextures.Add(MoveTemp(NewTexture));
	}

	return NewProxy;
}

FSlateShaderResourceProxy* FSlateOpenGLTextureManager::GetDynamicTextureResource(const FSlateBrush& InBrush)
{
	const FName ResourceName = InBrush.GetResourceName();

	// Bail out if we already have this texture loaded
	TSharedPtr<FDynamicTextureResource> TextureResource = DynamicTextureMap.FindRef(ResourceName);
	if (TextureResource.IsValid())
	{
		return TextureResource->Proxy;
	}

	if (InBrush.IsDynamicallyLoaded())
	{
		uint32 Width = 0;
		uint32 Height = 0;
		TArray<uint8> RawData;
		bool bSucceeded = LoadTexture(InBrush, Width, Height, RawData);

		if (bSucceeded)
		{
			return CreateDynamicTextureResource(ResourceName, Width, Height, RawData);
		}
		else
		{
			TextureResource = MakeShareable(new FDynamicTextureResource(NULL));

			// Add the null texture so we dont continuously try to load it.
			DynamicTextureMap.Add(ResourceName, TextureResource);
		}
	}

	if (TextureResource.IsValid())
	{
		return TextureResource->Proxy;
	}

	// dynamic texture was not found or loaded
	return  NULL;
}

FSlateShaderResourceProxy* FSlateOpenGLTextureManager::CreateDynamicTextureResource(FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& RawData)
{
	// Bail out if we already have this texture loaded
	TSharedPtr<FDynamicTextureResource> TextureResource = DynamicTextureMap.FindRef(ResourceName);
	if (TextureResource.IsValid())
	{
		return TextureResource->Proxy;
	}

	// Keep track of non-atlased textures so we can free their resources later
	FSlateOpenGLTexture* LoadedTexture = new FSlateOpenGLTexture(Width, Height);

	FNewTextureInfo Info;
	Info.bShouldAtlas = false;

#if !PLATFORM_USES_GLES
	LoadedTexture->Init(Info.bSrgb ? GL_SRGB8_ALPHA8 : GL_RGBA8, RawData);
#else
	LoadedTexture->Init(Info.bSrgb ? GL_SRGB8_ALPHA8_EXT : GL_RGBA8, RawData);
#endif

	TextureResource = MakeShareable(new FDynamicTextureResource(LoadedTexture));

	TextureResource->Proxy->ActualSize = FIntPoint(Width, Height);
	TextureResource->Proxy->Resource = TextureResource->OpenGLTexture;

	// Map the new resource for the UTexture so we don't have to load again
	DynamicTextureMap.Add(ResourceName, TextureResource);

	return TextureResource->Proxy;
}

void FSlateOpenGLTextureManager::ReleaseDynamicTextureResource(const FSlateBrush& InBrush)
{
	// Note: Only dynamically loaded or utexture brushes can be dynamically released
	if (InBrush.IsDynamicallyLoaded())
	{
		const FName ResourceName = InBrush.GetResourceName();
		TSharedPtr<FDynamicTextureResource> TextureResource = DynamicTextureMap.FindRef(ResourceName);
		if (TextureResource.IsValid())
		{
			//remove it from the texture map
			DynamicTextureMap.Remove(ResourceName);

			check(TextureResource.IsUnique());
		}
	}
}

void FSlateOpenGLTextureManager::UpdateCache()
{
	for (TUniquePtr<FSlateTextureAtlasOpenGL>& Atlas : PrecachedTextureAtlases)
	{
		Atlas->ConditionalUpdateTexture();
	}

	VectorGraphicsCache->UpdateCache();
}

void FSlateOpenGLTextureManager::ConditionalFlushCache()
{
	VectorGraphicsCache->ConditionalFlushCache();
}

/**
 * Returns a texture with the passed in name or NULL if it cannot be found.
 */
FSlateShaderResourceProxy* FSlateOpenGLTextureManager::GetShaderResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale)
{
	FSlateShaderResourceProxy* Texture = NULL;
	if (Brush.GetImageType() == ESlateBrushImageType::Vector)
	{
		Texture = GetVectorResource(Brush, LocalSize, DrawScale);
	}
	else if (Brush.IsDynamicallyLoaded())
	{
		Texture = GetDynamicTextureResource(Brush);
	}
	else
	{
		Texture = ResourceMap.FindRef(Brush.GetResourceName());
	}

	return Texture;
}

FSlateShaderResourceProxy* FSlateOpenGLTextureManager::GetVectorResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale)
{
	return VectorGraphicsCache->GetShaderResource(Brush, LocalSize, DrawScale);
}


ISlateAtlasProvider* FSlateOpenGLTextureManager::GetTextureAtlasProvider()
{
	return this;
}


int32 FSlateOpenGLTextureManager::GetNumAtlasPages() const
{
	return PrecachedTextureAtlases.Num() + VectorGraphicsCache->GetNumAtlasPages();
}

FSlateShaderResource* FSlateOpenGLTextureManager::GetAtlasPageResource(const int32 InIndex) const
{
	return InIndex < PrecachedTextureAtlases.Num() ? PrecachedTextureAtlases[InIndex]->GetAtlasTexture() : VectorGraphicsCache->GetAtlasPageResource(InIndex - PrecachedTextureAtlases.Num());
}


bool FSlateOpenGLTextureManager::IsAtlasPageResourceAlphaOnly(const int32 InIndex) const
{
	return false;
}

#if WITH_ATLAS_DEBUGGING
FAtlasSlotInfo FSlateOpenGLTextureManager::GetAtlasSlotInfoAtPosition(FIntPoint InPosition, int32 AtlasIndex) const
{
	const FSlateTextureAtlas* Atlas = nullptr;

	bool bIsPrecachedTextureAtlases = PrecachedTextureAtlases.IsValidIndex(AtlasIndex);
	if (bIsPrecachedTextureAtlases)
	{
		Atlas = PrecachedTextureAtlases[AtlasIndex].Get();
	}
	else
	{
		Atlas = VectorGraphicsCache->GetAtlas(AtlasIndex - PrecachedTextureAtlases.Num());
	}

	FAtlasSlotInfo NewInfo;
	if (Atlas)
	{
		const FAtlasedTextureSlot* Slot = Atlas->GetSlotAtPosition(InPosition);
		if (Slot)
		{
			NewInfo.AtlasSlotRect = FSlateRect(FVector2f(Slot->X, Slot->Y), FVector2f(Slot->X + Slot->Width, Slot->Y + Slot->Height));
			NewInfo.TextureName = bIsPrecachedTextureAtlases ? AtlasDebugData.FindRef(Slot) : VectorGraphicsCache->GetAtlasDebugData(Slot);
		}
	}

	return NewInfo;
}
#endif