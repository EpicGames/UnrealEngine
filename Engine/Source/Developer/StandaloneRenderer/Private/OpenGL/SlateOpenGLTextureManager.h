// Copyright Epic Games, Inc. All Rights Reserved.

/**
*/

#pragma once

#include "CoreMinimal.h"
#include "Rendering/ShaderResourceManager.h"

class FSlateOpenGLTexture;
class ISlateAtlasProvider;
class ISlateStyle;
class FSlateVectorGraphicsCache;
class FSlateTextureAtlasOpenGL;

/**
 * Stores a mapping of texture names to their loaded opengl resource.  Resources are loaded from disk and created on demand when needed
 */
class FSlateOpenGLTextureManager : public FSlateShaderResourceManager, public ISlateAtlasProvider
{
public:
	FSlateOpenGLTextureManager();
	virtual ~FSlateOpenGLTextureManager();

	void LoadUsedTextures();
	void LoadStyleResources(const ISlateStyle& Style);

	/**
	 * Returns a texture with the passed in name or NULL if it cannot be found.
	 */
	virtual FSlateShaderResourceProxy* GetShaderResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale) override;


	virtual ISlateAtlasProvider* GetTextureAtlasProvider() override;

	/** ISlateAtlasProvider */
	virtual int32 GetNumAtlasPages() const override;
	virtual FSlateShaderResource* GetAtlasPageResource(const int32 InIndex) const override;
	virtual bool IsAtlasPageResourceAlphaOnly(const int32 InIndex) const override;
#if WITH_ATLAS_DEBUGGING
	virtual FAtlasSlotInfo GetAtlasSlotInfoAtPosition(FIntPoint InPosition, int32 AtlasIndex) const override;
#endif

	/**
	 * Creates a 1x1 texture of the specified color
	 *
	 * @param TextureName	The name of the texture
	 * @param InColor		The color for the texture
	 */
	FSlateOpenGLTexture* CreateColorTexture(const FName TextureName, FColor InColor);

	FSlateShaderResourceProxy* CreateDynamicTextureResource(FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& RawData);

	void ReleaseDynamicTextureResource(const FSlateBrush& InBrush);

	void UpdateCache();
	void ConditionalFlushCache();
private:
	/**
	 * Creates textures from files on disk and atlases them if possible
	 *
	 * @param Resources	The brush resources to load images for
	 */
	void CreateTextures(const TArray< const FSlateBrush* >& Resources);

	bool LoadTexture(const FSlateBrush& InBrush, uint32& OutWidth, uint32& OutHeight, TArray<uint8>& OutDecodedImage);

	FSlateShaderResourceProxy* GenerateTextureResource(const FNewTextureInfo& Info, FName TextureName);

	FSlateShaderResourceProxy* GetDynamicTextureResource(const FSlateBrush& InBrush);

	FSlateShaderResourceProxy* GetVectorResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale);
private:
	/** Represents a dynamic resource for rendering */
	struct FDynamicTextureResource
	{
		FSlateShaderResourceProxy* Proxy;
		FSlateOpenGLTexture* OpenGLTexture;

		FDynamicTextureResource(FSlateOpenGLTexture* OpenGLTexture);
		~FDynamicTextureResource();
	};

	/** Map of all active dynamic texture objects being used for brush resources */
	TMap<FName, TSharedPtr<FDynamicTextureResource> > DynamicTextureMap;

	/** Static texture atlases */
	TArray<TUniquePtr<FSlateTextureAtlasOpenGL>> PrecachedTextureAtlases;

	/** Static non atlased textures */
	TArray<TUniquePtr<FSlateOpenGLTexture>> NonAtlasedTextures;

	TUniquePtr<FSlateVectorGraphicsCache> VectorGraphicsCache;
};

