// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenGL/SlateOpenGLTextures.h"
#include "OpenGL/SlateOpenGLRenderer.h"
#if PLATFORM_MAC
#include "Mac/OpenGL/SlateOpenGLMac.h"
#endif
#define USE_DEPRECATED_OPENGL_FUNCTIONALITY			(!PLATFORM_USES_GLES && !PLATFORM_LINUX)

GLuint FSlateOpenGLTexture::NullTexture = 0;

void FSlateOpenGLTexture::Init( GLenum InTexFormat, const TArray<uint8>& TextureData )
{
#if PLATFORM_MAC
	LockGLContext([NSOpenGLContext currentContext]);
#endif
	// Create a new OpenGL texture
	glGenTextures(1, &ShaderResource);
	CHECK_GL_ERRORS;

	// Ensure texturing is enabled before setting texture properties
#if USE_DEPRECATED_OPENGL_FUNCTIONALITY
	glEnable(GL_TEXTURE_2D);
#endif // USE_DEPRECATED_OPENGL_FUNCTIONALITY
	glBindTexture(GL_TEXTURE_2D, ShaderResource);
	glTexParameteri(GL_TEXTURE_2D, GL_TEXTURE_MIN_FILTER, GL_LINEAR);
	glTexParameteri(GL_TEXTURE_2D, GL_TEXTURE_MAG_FILTER, GL_LINEAR);
#if USE_DEPRECATED_OPENGL_FUNCTIONALITY
	glTexEnvi( GL_TEXTURE_ENV, GL_TEXTURE_ENV_MODE, GL_REPLACE );
#endif // USE_DEPRECATED_OPENGL_FUNCTIONALITY

	// the raw data is in bgra or bgr
	const GLint Format = GL_RGBA;

	TexFormat = InTexFormat;

	// Upload the texture data
	glTexImage2D( GL_TEXTURE_2D, 0, TexFormat, SizeX, SizeY, 0, Format, GL_UNSIGNED_INT_8_8_8_8_REV, TextureData.GetData() );
	bHasPendingResize = false;
	CHECK_GL_ERRORS;
#if PLATFORM_MAC
	UnlockGLContext([NSOpenGLContext currentContext]);
#endif
}

void FSlateOpenGLTexture::Init( GLuint TextureID )
{
	ShaderResource = TextureID;
	bHasPendingResize = false;
}

void FSlateOpenGLTexture::Init( void* TextureHandle )
{
#if PLATFORM_MAC
	LockGLContext([NSOpenGLContext currentContext]);

	// Create a new OpenGL texture
	glGenTextures(1, &ShaderResource);
	CHECK_GL_ERRORS;
	
	glBindTexture(GL_TEXTURE_RECTANGLE_ARB, ShaderResource);
	glTexParameteri(GL_TEXTURE_RECTANGLE_ARB, GL_TEXTURE_MIN_FILTER, GL_LINEAR);
	glTexParameteri(GL_TEXTURE_RECTANGLE_ARB, GL_TEXTURE_MAG_FILTER, GL_LINEAR);

	IOSurfaceRef LastHandle = (IOSurfaceRef)TextureHandle;
	
	CGLContextObj cglContext = CGLGetCurrentContext();
	
	SizeX = IOSurfaceGetWidth(LastHandle);
	SizeY = IOSurfaceGetHeight(LastHandle);
	
	bHasPendingResize = false;
	UnlockGLContext([NSOpenGLContext currentContext]);
#else
	checkf( false, TEXT("Needs Implementation") );
#endif
}


void FSlateOpenGLTexture::ResizeTexture(uint32 Width, uint32 Height)
{
	SizeX = Width;
	SizeY = Height;
	bHasPendingResize = true;
}

void FSlateOpenGLTexture::UpdateTexture(const TArray<uint8>& Bytes)
{
	UpdateTextureRaw(Bytes.GetData(), FIntRect());
}

void FSlateOpenGLTexture::UpdateTextureThreadSafeRaw(uint32 Width, uint32 Height, const void* Buffer, const FIntRect& Dirty)
{
	if (SizeX != Width || SizeY != Height)
	{
		ResizeTexture(Width, Height);
	}
	UpdateTextureRaw(Buffer, Dirty);
}

void FSlateOpenGLTexture::UpdateTextureThreadSafeWithTextureData(FSlateTextureData* TextureData)
{
	UpdateTextureThreadSafeRaw(TextureData->GetWidth(), TextureData->GetHeight(), TextureData->GetRawBytesPtr());
	delete TextureData;
}


void FSlateOpenGLTexture::UpdateTextureRaw(const void* Buffer, const FIntRect& Dirty)
{
#if PLATFORM_MAC
	LockGLContext([NSOpenGLContext currentContext]);
#endif
	// Ensure texturing is enabled before setting texture properties
#if USE_DEPRECATED_OPENGL_FUNCTIONALITY
	glEnable(GL_TEXTURE_2D);
#endif // USE_DEPRECATED_OPENGL_FUNCTIONALITY
	glBindTexture(GL_TEXTURE_2D, ShaderResource);
	glTexParameteri(GL_TEXTURE_2D, GL_TEXTURE_MIN_FILTER, GL_LINEAR);
	glTexParameteri(GL_TEXTURE_2D, GL_TEXTURE_MAG_FILTER, GL_LINEAR);
	glTexParameteri(GL_TEXTURE_2D, GL_TEXTURE_WRAP_S, GL_CLAMP_TO_EDGE);
#if USE_DEPRECATED_OPENGL_FUNCTIONALITY
	glTexEnvi(GL_TEXTURE_ENV, GL_TEXTURE_ENV_MODE, GL_REPLACE);
#endif // USE_DEPRECATED_OPENGL_FUNCTIONALITY
	
	// Upload the texture data
#if !PLATFORM_USES_GLES

	if (bHasPendingResize || Dirty.Area() == 0)
	{
		glTexImage2D(GL_TEXTURE_2D, 0, TexFormat, SizeX, SizeY, 0, GL_RGBA, GL_UNSIGNED_INT_8_8_8_8_REV, Buffer);
		bHasPendingResize = false;
	}
	else
	{
		glPixelStorei(GL_UNPACK_ROW_LENGTH, SizeX);
		glTexSubImage2D(GL_TEXTURE_2D, 0, Dirty.Min.X, Dirty.Min.Y, Dirty.Width(), Dirty.Height(), TexFormat, GL_UNSIGNED_INT_8_8_8_8_REV, (uint8*)Buffer + Dirty.Min.Y * SizeX * 4 + Dirty.Min.X * 4);
		glPixelStorei(GL_UNPACK_ROW_LENGTH, 0);
	}
#else
	glTexImage2D(GL_TEXTURE_2D, 0, TexFormat, SizeX, SizeY, 0, GL_RGBA, GL_UNSIGNED_INT_8_8_8_8_REV, Buffer);
#endif
	CHECK_GL_ERRORS;
#if PLATFORM_MAC
	UnlockGLContext([NSOpenGLContext currentContext]);
#endif
}

#if PLATFORM_MAC
void FSlateOpenGLTexture::UpdateTextureThreadSafeWithKeyedTextureHandle(void* TextureHandle, int KeyLockVal, int KeyUnlockVal, const FIntRect& Dirty)
{
	LockGLContext([NSOpenGLContext currentContext]);
	glEnable(GL_TEXTURE_RECTANGLE_ARB);
	glActiveTexture(GL_TEXTURE1);
	
	glBindTexture(GL_TEXTURE_RECTANGLE_ARB, ShaderResource);
	glTexParameteri(GL_TEXTURE_RECTANGLE_ARB, GL_TEXTURE_MIN_FILTER, GL_LINEAR);
	glTexParameteri(GL_TEXTURE_RECTANGLE_ARB, GL_TEXTURE_MAG_FILTER, GL_LINEAR);
	glTexParameteri(GL_TEXTURE_RECTANGLE_ARB, GL_TEXTURE_WRAP_S, GL_CLAMP_TO_EDGE);
	//glPixelStorei(GL_UNPACK_ALIGNMENT, 1);
	CHECK_GL_ERRORS;

	IOSurfaceRef LastHandle = (IOSurfaceRef)TextureHandle;
	if (LastHandle != nullptr)
	{
		//IOSurfaceUnlock(LastHandle, 0, NULL);
		
		CGLContextObj cglContext = CGLGetCurrentContext();
		
		GLsizei SurfaceWidth = IOSurfaceGetWidth(LastHandle);
		GLsizei SurfaceHeight = IOSurfaceGetHeight(LastHandle);
		
		if (SurfaceWidth != SizeX || SurfaceHeight != SizeY )
		{
			//fprintf( stderr, "Buffer Size mistmatch: %d %d %d %d\n", SurfaceWidth, SizeY, SurfaceHeight, SizeX );
			SizeX = SurfaceWidth;
			SizeY = SurfaceHeight;
		}
		bHasPendingResize = false; // this always resizes our texture
		
		CGLError cglError = CGLTexImageIOSurface2D(cglContext, GL_TEXTURE_RECTANGLE_ARB, GL_SRGB,
												   SurfaceWidth, SurfaceHeight, GL_BGRA,
													GL_UNSIGNED_INT_8_8_8_8_REV, LastHandle, 0);
		checkf( cglError == kCGLNoError, TEXT("CGL error: 0x%x"), cglError );
	}

	glActiveTexture(GL_TEXTURE0);

	UnlockGLContext([NSOpenGLContext currentContext]);
}
#endif

FSlateFontTextureOpenGL::FSlateFontTextureOpenGL(uint32 Width, uint32 Height, ESlateFontAtlasContentType InContentType)
	: FSlateFontAtlas(Width, Height, InContentType, NoPadding)
	, FontTexture(nullptr)
{
}

FSlateFontTextureOpenGL::~FSlateFontTextureOpenGL()
{
	delete FontTexture;
}

void FSlateFontTextureOpenGL::CreateFontTexture()
{
	// Generate an ID for this texture
	GLuint TextureID;
	glGenTextures(1, &TextureID);

	// Bind the texture so we can specify filtering and the data to use
	glBindTexture(GL_TEXTURE_2D, TextureID);
	glTexParameteri(GL_TEXTURE_2D, GL_TEXTURE_MIN_FILTER, GL_NEAREST);
	glTexParameteri(GL_TEXTURE_2D, GL_TEXTURE_MAG_FILTER, GL_NEAREST);

	GLint InternalFormat = GetGLTextureInternalFormat();
	GLint Format = GetGLTextureFormat();
	GLint Type = GetGLTextureType();

	// Upload the data to the texture
	glTexImage2D( GL_TEXTURE_2D, 0, InternalFormat, AtlasWidth, AtlasHeight, 0, Format, Type, NULL );

	// Create a new slate texture for use in rendering
	FontTexture = new FSlateOpenGLTexture( AtlasWidth, AtlasHeight );
	FontTexture->Init( TextureID );
}

void FSlateFontTextureOpenGL::ConditionalUpdateTexture()
{
	// The texture may not be valid when calling this as OpenGL must wait until after the first viewport has been created to create a texture
	if( bNeedsUpdate && FontTexture )
	{
		check(AtlasData.Num()>0);

		// Completely the texture data each time characters are added
		glBindTexture(GL_TEXTURE_2D, FontTexture->GetTypedResource() );

#if PLATFORM_MAC // Make this texture use a DMA'd client storage backing store on OS X, where these extensions always exist
				 // This avoids a problem on Intel & Nvidia cards that makes characters disappear as well as making the texture updates
				 // as fast as they possibly can be.
		glTextureRangeAPPLE(GL_TEXTURE_2D, AtlasData.Num(), AtlasData.GetData());
		glTexParameteri(GL_TEXTURE_2D, GL_TEXTURE_STORAGE_HINT_APPLE, GL_STORAGE_CACHED_APPLE);
		glPixelStorei(GL_UNPACK_CLIENT_STORAGE_APPLE, GL_TRUE);
#endif
		GLint InternalFormat = GetGLTextureInternalFormat();
		GLint Format = GetGLTextureFormat();
		GLint Type = GetGLTextureType();
		glTexImage2D( GL_TEXTURE_2D, 0, InternalFormat, AtlasWidth, AtlasHeight, 0, Format, Type, AtlasData.GetData() );
#if PLATFORM_MAC
		glPixelStorei(GL_UNPACK_CLIENT_STORAGE_APPLE, GL_FALSE);
#endif
		
		bNeedsUpdate = false;
	}
}

GLint FSlateFontTextureOpenGL::GetGLTextureInternalFormat() const
{
	switch (GetContentType())
	{
		case ESlateFontAtlasContentType::Alpha:
#if USE_DEPRECATED_OPENGL_FUNCTIONALITY
			return GL_ALPHA;
#else
			return GL_RED;
#endif // USE_DEPRECATED_OPENGL_FUNCTIONALITY
		default:
			checkNoEntry();
			// Default to Color
			// falls through
		case ESlateFontAtlasContentType::Color:
#if !PLATFORM_USES_GLES
			return GL_SRGB8_ALPHA8;
#else
			return GL_SRGB8_ALPHA8_EXT;
#endif
		case ESlateFontAtlasContentType::Msdf:
			return GL_RGBA;
	}
}

GLint FSlateFontTextureOpenGL::GetGLTextureFormat() const
{
	switch (GetContentType())
	{
		case ESlateFontAtlasContentType::Alpha:
#if USE_DEPRECATED_OPENGL_FUNCTIONALITY
			return GL_ALPHA;
#else
			return GL_RED;
#endif // USE_DEPRECATED_OPENGL_FUNCTIONALITY
		case ESlateFontAtlasContentType::Color:
		case ESlateFontAtlasContentType::Msdf:
			return GL_RGBA;
		default:
			checkNoEntry();
			// Default to Color
			return GL_RGBA;
	}
}

GLint FSlateFontTextureOpenGL::GetGLTextureType() const
{
	switch (GetContentType())
	{
		case ESlateFontAtlasContentType::Alpha:
			return GL_UNSIGNED_BYTE;
		case ESlateFontAtlasContentType::Color:
		case ESlateFontAtlasContentType::Msdf:
			return GL_UNSIGNED_INT_8_8_8_8_REV;
		default:
			checkNoEntry();
			// Default to Color
			return GL_UNSIGNED_INT_8_8_8_8_REV;
	}
}

FSlateTextureAtlasOpenGL::FSlateTextureAtlasOpenGL(uint32 Width, uint32 Height, uint32 StrideBytes, ESlateTextureAtlasPaddingStyle PaddingStyle)
	: FSlateTextureAtlas(Width, Height, StrideBytes, PaddingStyle, true)
{
	InitAtlasTexture();
}

FSlateTextureAtlasOpenGL::~FSlateTextureAtlasOpenGL()
{
	if (AtlasTexture)
	{
		delete AtlasTexture;
	}
}

void FSlateTextureAtlasOpenGL::ConditionalUpdateTexture()
{
	if (bNeedsUpdate)
	{
		AtlasTexture->UpdateTexture(AtlasData);
		bNeedsUpdate = false;
	}
}

void FSlateTextureAtlasOpenGL::InitAtlasTexture()
{
	AtlasTexture = new FSlateOpenGLTexture(AtlasWidth, AtlasHeight);
			
#if !PLATFORM_USES_GLES
	AtlasTexture->Init(GL_SRGB8_ALPHA8, TArray<uint8>());
#else
	AtlasTexture->Init(GL_SRGB8_ALPHA8_EXT, TArray<uint8>());
#endif
}
