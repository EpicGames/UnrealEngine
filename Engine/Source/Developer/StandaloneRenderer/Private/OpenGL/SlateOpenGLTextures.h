// Copyright Epic Games, Inc. All Rights Reserved.

/**
*/

#pragma once

#include "CoreMinimal.h"
#include "Textures/SlateShaderResource.h"
#include "Fonts/FontTypes.h"
#include "Textures/SlateUpdatableTexture.h"
#include "OpenGL/SlateOpenGLExtensions.h"

struct FSlateTextureData;

class FSlateOpenGLTexture : public TSlateTexture< GLuint >, public FSlateUpdatableTexture
{
public:
	FSlateOpenGLTexture( uint32 InSizeX, uint32 InSizeY )
		: TSlateTexture( FSlateOpenGLTexture::NullTexture )
		, SizeX(InSizeX)
		, SizeY(InSizeY)
		, TextureTargetType(GL_TEXTURE_2D)
	{

	}

	FSlateOpenGLTexture()
	: TSlateTexture( FSlateOpenGLTexture::NullTexture )
	, SizeX(0)
	, SizeY(0)
#if PLATFORM_LINUX
	, TextureTargetType(GL_TEXTURE_RECTANGLE)
#else
	, TextureTargetType(GL_TEXTURE_RECTANGLE_ARB)
#endif
	{
		
	}

	void Init( GLenum InTexFormat, const TArray<uint8>& TextureData );

	void Init( GLuint TextureID );

	void Init( void* TextureHandle );

	~FSlateOpenGLTexture()
	{
		glDeleteTextures( 1, &ShaderResource );
	}

	virtual void Cleanup() override { delete this; }

	uint32 GetWidth() const { return SizeX; }
	uint32 GetHeight() const { return SizeY; }

	// FSlateUpdatableTexture interface
	virtual FSlateShaderResource* GetSlateResource() override {return this;}
	virtual void ResizeTexture( uint32 Width, uint32 Height ) override;
	virtual void UpdateTexture(const TArray<uint8>& Bytes) override;
	virtual void UpdateTextureThreadSafe(const TArray<uint8>& Bytes) override { UpdateTexture(Bytes); }
	virtual void UpdateTextureThreadSafeRaw(uint32 Width, uint32 Height, const void* Buffer, const FIntRect& Dirty = FIntRect()) override;
	virtual void UpdateTextureThreadSafeWithTextureData(FSlateTextureData* TextureData) override;
#if PLATFORM_MAC
	// only macOS has support for texture handles currently
	virtual void UpdateTextureThreadSafeWithKeyedTextureHandle(void* TextureHandle, int KeyLockVal, int KeyUnlockVal, const FIntRect& Dirty = FIntRect()) override;
#else
	virtual void UpdateTextureThreadSafeWithKeyedTextureHandle(void* TextureHandle, int KeyLockVal, int KeyUnlockVal, const FIntRect& Dirty = FIntRect()) override {}
#endif
	
	uint32 GetTextureTargetType() const { return TextureTargetType; }
private:
	// Helper method used by the different UpdateTexture* methods
	void UpdateTextureRaw(const void* Buffer, const FIntRect& Dirty);

	static GLuint NullTexture;

	GLenum TexFormat;
	uint32 SizeX;
	uint32 SizeY;
	bool bHasPendingResize;
	uint32 TextureTargetType;
};

/** 
 * Representation of a texture for fonts in which characters are packed tightly based on their bounding rectangle 
 */
class FSlateFontTextureOpenGL : public FSlateFontAtlas
{
public:
	FSlateFontTextureOpenGL(uint32 Width, uint32 Height, ESlateFontAtlasContentType InContentType);
	~FSlateFontTextureOpenGL();

	void CreateFontTexture();

	/** FSlateFontAtlas interface */
	virtual void ConditionalUpdateTexture();
	virtual class FSlateShaderResource* GetSlateTexture() const override { return FontTexture; }
	virtual class FTextureResource* GetEngineTexture() override { return nullptr; }
	virtual void ReleaseResources() override {}

private:
	GLint GetGLTextureInternalFormat() const;
	GLint GetGLTextureFormat() const;
	GLint GetGLTextureType() const;

	FSlateOpenGLTexture* FontTexture;
};


/**
 * Representation of a texture for images in which characters are packed tightly based on their bounding rectangle
 */
class FSlateTextureAtlasOpenGL : public FSlateTextureAtlas
{
public:
	FSlateTextureAtlasOpenGL(uint32 Width, uint32 Height, uint32 StrideBytes, ESlateTextureAtlasPaddingStyle PaddingStyle);
	~FSlateTextureAtlasOpenGL();


	virtual void ConditionalUpdateTexture() override;
	virtual void ReleaseResources() override {}

	FSlateOpenGLTexture* GetAtlasTexture() const { return AtlasTexture; }
private:
	void InitAtlasTexture();
private:
	FSlateOpenGLTexture* AtlasTexture;
	GLuint TextureID;
};


