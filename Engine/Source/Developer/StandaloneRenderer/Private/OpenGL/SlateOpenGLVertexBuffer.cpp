// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenGL/SlateOpenGLVertexBuffer.h"
#include "OpenGL/SlateOpenGLExtensions.h"

#include "OpenGL/SlateOpenGLRenderer.h"

FSlateOpenGLVertexBuffer::FSlateOpenGLVertexBuffer( uint32 InStride )
	: BufferSize(0)
	, Stride(InStride)
	, BufferID(0)
{
	check( InStride > 0 );
}

FSlateOpenGLVertexBuffer::~FSlateOpenGLVertexBuffer()
{
	DestroyBuffer();
}

/** 
 * Initializes the vertex buffers resource if needed
 */
void FSlateOpenGLVertexBuffer::ConditionalCreateBuffer()
{
	// Only generate the buffer if we don't have a valid one
	if( !IsValid() )
	{
		glGenBuffers( 1, &BufferID );
	}
	CHECK_GL_ERRORS;
}

/** 
 * Releases the vertex buffers resource. 
 */
void FSlateOpenGLVertexBuffer::DestroyBuffer()
{
	if( IsValid() )
	{
		glDeleteBuffers( 1, &BufferID );
		BufferID = 0;
	}
}

/** 
 * Resizes the buffer to the passed in number of indices.  
 * Preserves internal data
 */
void FSlateOpenGLVertexBuffer::ResizeBuffer( uint32 NewSize )
{
	ConditionalCreateBuffer();

	// Only resize if the buffer cant provide the number of bytes requestd
	if( NewSize > BufferSize )
	{
		uint8 *SavedVertices = NULL;

		// If there are any indices at all save them off now
		if( BufferSize > 0 )
		{
			void *Vertices = Lock( 0 );

			SavedVertices = new uint8[BufferSize];
			FMemory::Memcpy( SavedVertices, Vertices, BufferSize );

			Unlock();

			DestroyBuffer();
		}

		// Create the vertex buffer
		ConditionalCreateBuffer();

		// Bind the buffer so we can give it data
		Bind();

		// Set the vertex buffer's size
		glBufferData( GL_ARRAY_BUFFER, NewSize, NULL, GL_DYNAMIC_DRAW );

		// If there are any saved vertices, copy them back now.
		if( SavedVertices )
		{
			void *Vertices = Lock( 0 );

			FMemory::Memcpy( Vertices, SavedVertices, BufferSize );

			Unlock();

			delete[] SavedVertices;
		}

		BufferSize = NewSize;
	}

}

/** 
 * Locks the vertex buffer, returning a pointer to its vertices
 */
void* FSlateOpenGLVertexBuffer::Lock( uint32 Offset )
{
	// Bind the vertex buffer so we can access its data
	Bind();

	// Map the buffer data
	// Calling glBufferData here is the same as a discard.  We avoid a pipeline flush
	glBufferData( GL_ARRAY_BUFFER, BufferSize, NULL, GL_DYNAMIC_DRAW );
	void* Data = glMapBuffer( GL_ARRAY_BUFFER, GL_WRITE_ONLY );
	CHECK_GL_ERRORS;

	check(Data);

	return (void*)((uint8*)Data + Offset);
}

/** 
 * Unlocks the buffer.  Pointers to buffer data will no longer be valid after this call
 */
void FSlateOpenGLVertexBuffer::Unlock()
{
	Bind();
	glUnmapBuffer(GL_ARRAY_BUFFER);
	CHECK_GL_ERRORS;
}

/** 
 * Binds the buffer so it can be accessed
 */
void FSlateOpenGLVertexBuffer::Bind()
{
	glBindBuffer( GL_ARRAY_BUFFER, BufferID );
	CHECK_GL_ERRORS;
}
