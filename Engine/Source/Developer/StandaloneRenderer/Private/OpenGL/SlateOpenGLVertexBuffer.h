// Copyright Epic Games, Inc. All Rights Reserved.

/**
*/

#pragma once

#include "CoreMinimal.h"
#include "StandaloneRendererPlatformHeaders.h"

/** 
 * An OpenGL vertex buffer 
 */
class FSlateOpenGLVertexBuffer
{
public:
	FSlateOpenGLVertexBuffer( uint32 InStride );
	~FSlateOpenGLVertexBuffer();

	/** 
	 * Releases the vertex buffers resource. 
	 */
	void DestroyBuffer();

	/** 
	 * Returns the size of the buffer in bytes. 
	 */
	uint32 GetBufferSize() const { return BufferSize; }

	/** 
	 * Resizes the buffer to the passed in number of indices.  
	 * Preserves internal data
	 */
	void ResizeBuffer( uint32 NewSize );

	/** 
	 * Locks the vertex buffer, returning a pointer to its vertices
	 */
	void* Lock( uint32 Offset );

	/** 
	 * Unlocks the buffer.  Pointers to buffer data will no longer be valid after this call
	 */
	void Unlock();

	/** 
	 * Binds the buffer so it can be accessed
	 */
	void Bind();

	/**
	 * Returns true if the buffer is valid and can be used                   
	 */
	bool IsValid() const { return BufferID != 0; }
private:
	/** 
	 * Initializes the vertex buffers resource if needed
	 */
	void ConditionalCreateBuffer();
private:
	/** The size of the buffer in bytes. */
	GLuint BufferSize;
	/** The size of each element in the buffer. */
	GLuint Stride;
	/** An OpenGL resource ID for the buffer */
	GLuint BufferID;
	/** Hidden copy methods. */
	FSlateOpenGLVertexBuffer( const FSlateOpenGLVertexBuffer& );
	void operator=(const FSlateOpenGLVertexBuffer& );
};
