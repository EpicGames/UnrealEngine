// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandaloneRenderer.h"
#include "Interfaces/ISlateNullRendererModule.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Styling/CoreStyle.h"
#include "StandaloneRendererLog.h"

#if PLATFORM_WINDOWS
#include "Windows/D3D/SlateD3DRenderer.h"
#endif

#include "Rendering/SlateRenderer.h"
#include "OpenGL/SlateOpenGLRenderer.h"

DEFINE_LOG_CATEGORY(LogStandaloneRenderer);

/**
 * Single function to create the standalone renderer for the running platform
 */
TSharedRef<FSlateRenderer> GetStandardStandaloneRenderer()
{
	// create a standalone renderer object
	TSharedPtr<FSlateRenderer> Renderer = NULL;
	if (FApp::CanEverRender())
	{
#if PLATFORM_WINDOWS
		bool bUseOpenGL = FParse::Param( FCommandLine::Get(), TEXT("opengl") );
		if( bUseOpenGL )
		{
#endif
			Renderer = TSharedPtr<FSlateRenderer>( new FSlateOpenGLRenderer( FCoreStyle::Get() ) );
#if PLATFORM_WINDOWS
		}
		else
		{
			Renderer = TSharedPtr<FSlateRenderer>( new FSlateD3DRenderer( FCoreStyle::Get() ) );
		}
#endif
	}
	else
	{
		Renderer = FModuleManager::Get().LoadModuleChecked<ISlateNullRendererModule>("SlateNullRenderer").CreateSlateNullRenderer();
	}

	// enforce non-NULL pointer
	return Renderer.ToSharedRef();
}

class FStandaloneRenderer : public IModuleInterface
{
};

IMPLEMENT_MODULE( FStandaloneRenderer, StandaloneRenderer )
