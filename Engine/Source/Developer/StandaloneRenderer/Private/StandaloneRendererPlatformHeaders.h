// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"

// D3D headers.


// Disable macro redefinition warning for compatibility with Windows SDK 8+
#pragma warning(push)
#pragma warning(disable : 4005)	// macro redefinition

#include <d3d11.h>
#include <d3d11_1.h>
#include <D3Dcompiler.h>

#pragma warning(pop)

#include "Windows/HideWindowsPlatformTypes.h"
#endif		// PLATFORM_WINDOWS

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include <GL/gl.h>
#include <GL/glext.h>
#include <GL/wglext.h>
#include "Windows/HideWindowsPlatformTypes.h"
#elif PLATFORM_MAC
#include <OpenGL/gl.h>
#include <OpenGL/glext.h>
#elif PLATFORM_IOS

#include <OpenGLES/ES2/gl.h>
#include <OpenGLES/ES2/glext.h>

// renamed keywords
#define glMapBuffer		glMapBufferOES
#define glUnmapBuffer	glUnmapBufferOES
#define GL_WRITE_ONLY	GL_WRITE_ONLY_OES
#define GL_CLAMP		GL_CLAMP_TO_EDGE
#define GL_RGBA8		GL_RGBA
#define GL_UNSIGNED_INT_8_8_8_8_REV	GL_UNSIGNED_BYTE

// unused functions
#define glTexEnvf(...)
#define glTexEnvi(...)
#define glAlphaFunc(...)

#elif PLATFORM_LINUX
//#define GLCOREARB_PROTOTYPES 
#include <GL/glcorearb.h>
//#include <GL/glext.h>
#include <SDL3/SDL.h>

#endif // PLATFORM_LINUX
