// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include "StandaloneRendererPlatformHeaders.h"
#include "Windows/D3D/SlateD3DRenderer.h"

template<typename BufferType>
class FSlateD3DConstantBuffer
{
public:
	void Create()
	{
		D3D11_BUFFER_DESC Desc;
		Desc.BindFlags = D3D11_BIND_CONSTANT_BUFFER;
		Desc.Usage = D3D11_USAGE_DYNAMIC;
		Desc.ByteWidth = sizeof(BufferType);
		Desc.CPUAccessFlags = D3D11_CPU_ACCESS_WRITE;
		Desc.MiscFlags = 0;
		Desc.StructureByteStride = 0;

		HRESULT Hr = GD3DDevice->CreateBuffer(&Desc, NULL, Buffer.GetInitReference() );
		if (FAILED(Hr))
		{
			LogSlateD3DRendererFailure(TEXT("FSlateD3DConstantBuffer::Create() - ID3D11Device::CreateBuffer"), Hr);
			GEncounteredCriticalD3DDeviceError = true;
			return;
		}
	}

	TRefCountPtr<ID3D11Buffer> GetResource() { return Buffer; }
	BufferType& GetBufferData() { return BufferData; }
	void UpdateBuffer()
	{
		void* Data = Lock();
		FMemory::Memcpy(Data, &BufferData, sizeof(BufferType) );
		Unlock();
	}
private:
	void* Lock()
	{
		D3D11_MAPPED_SUBRESOURCE Resource;
		GD3DDeviceContext->Map( Buffer, 0, D3D11_MAP_WRITE_DISCARD, 0, &Resource );
		return Resource.pData;
	}

	void Unlock()
	{		
		GD3DDeviceContext->Unmap( Buffer, 0 );
	}
private:
	TRefCountPtr<ID3D11Buffer> Buffer;
	BufferType BufferData;
};

