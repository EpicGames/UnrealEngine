// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/D3D/SlateD3DFindBestDevice.h"

#include "Misc/CommandLine.h"
#include "StandaloneRendererPlatformHeaders.h"
#include "StandaloneRendererLog.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/D3D/SlateD3DRenderer.h"
#include "Windows/WindowsPlatformCrashContext.h"
#include <delayimp.h>
THIRD_PARTY_INCLUDES_START
//#include <d3d11_2.h>
#include <dxgi1_6.h>
#include <dxgidebug.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

namespace UE::StandaloneRenderer::D3D
{
namespace Private
{

bool IsDelayLoadException(PEXCEPTION_POINTERS ExceptionPointers)
{
	switch (ExceptionPointers->ExceptionRecord->ExceptionCode)
	{
	case VcppException(ERROR_SEVERITY_ERROR, ERROR_MOD_NOT_FOUND):
	case VcppException(ERROR_SEVERITY_ERROR, ERROR_PROC_NOT_FOUND):
		return EXCEPTION_EXECUTE_HANDLER;
	default:
		return EXCEPTION_CONTINUE_SEARCH;
	}
}

} //namespace Private


FFindBestDevice::FFindBestDevice()
{
	ChosenFeatureLevel = D3D_FEATURE_LEVEL_1_0_CORE;
	GpuPreference = DXGI_GPU_PREFERENCE_UNSPECIFIED;
	LoadSettings();
	SelectAdapter();
}

FFindBestDevice::~FFindBestDevice()
{}

bool FFindBestDevice::IsValid() const
{
	return ChosenAdapter.IsValid() && ChosenFeatureLevel != D3D_FEATURE_LEVEL_1_0_CORE;
}

bool FFindBestDevice::CreateDevice(ID3D11Device** Device, ID3D11DeviceContext** DeviceContext)
{
	if (!IsValid())
	{
		return false;
	}

	// Init D3D
	D3D_DRIVER_TYPE DriverType = D3D_DRIVER_TYPE_HARDWARE;

	uint32 DeviceFlags = D3D11_CREATE_DEVICE_SINGLETHREADED;
	if (bDebug)
	{
		DeviceFlags |= D3D11_CREATE_DEVICE_DEBUG;
	}

	D3D_FEATURE_LEVEL CreatedFeatureLevel = D3D_FEATURE_LEVEL_1_0_CORE;
	HRESULT Hr = D3D11CreateDevice(ChosenAdapter, D3D_DRIVER_TYPE_UNKNOWN, nullptr, DeviceFlags, &ChosenFeatureLevel, 1, D3D11_SDK_VERSION, Device, &CreatedFeatureLevel, DeviceContext);
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DRenderer::CreateDevice() - D3D11CreateDevice"), Hr);
	}
	check(CreatedFeatureLevel == ChosenFeatureLevel);
	return SUCCEEDED(Hr);
}

const TCHAR* FFindBestDevice::GetFeatureLevelString(D3D_FEATURE_LEVEL FeatureLevel)
{
	// Note: this switch only needs the feature levels used inside FFindBestDevice::CreateTesingDevice
	switch (FeatureLevel)
	{
	case D3D_FEATURE_LEVEL_11_0:	return TEXT("11_0");
	case D3D_FEATURE_LEVEL_11_1:	return TEXT("11_1");
	}
	return TEXT("X_X");
}

void FFindBestDevice::LoadSettings()
{
	FParse::Value(FCommandLine::Get(), TEXT("-ExplicitAdapterValue="), ExplicitAdapterValue);
	FParse::Value(FCommandLine::Get(), TEXT("-PreferedAdapterVendor="), PreferedAdapterVendor);
	bAllowSoftwareRendering = FParse::Param(FCommandLine::Get(), TEXT("AllowSoftwareRendering"));
	bPreferedMinimalPower = FParse::Param(FCommandLine::Get(), TEXT("PreferedMinimalPower"));
	bPreferedHighPerformance = FParse::Param(FCommandLine::Get(), TEXT("PreferedHighPerformance"));
	bDebug = FParse::Param(FCommandLine::Get(), TEXT("d3ddebug"));

	CreateDXGIFactory1(__uuidof(IDXGIFactory1), (void**)DXGIFactory1.GetInitReference());
	if (DXGIFactory1)
	{
		DXGIFactory1->QueryInterface(__uuidof(IDXGIFactory6), (void**)DXGIFactory6.GetInitReference());
	}

	if (bPreferedMinimalPower)
	{
		GpuPreference = DXGI_GPU_PREFERENCE_MINIMUM_POWER;
	}
	else
	{
		GpuPreference = DXGI_GPU_PREFERENCE_HIGH_PERFORMANCE;
	}

	UE_LOGF(LogStandaloneRenderer, Log, "D3D11 adapters settings:");
	if (ExplicitAdapterValue >= 0)
	{
		UE_LOGF(LogStandaloneRenderer, Log, "ExplicitAdapterValue=%d", ExplicitAdapterValue);
	}
	if (PreferedAdapterVendor >= 0)
	{
		UE_LOGF(LogStandaloneRenderer, Log, "PreferedAdapterVendor=%d", PreferedAdapterVendor);
	}
	if (bAllowSoftwareRendering)
	{
		UE_LOGF(LogStandaloneRenderer, Log, "bAllowSoftwareRendering=%ls", (bAllowSoftwareRendering ? TEXT("true") : TEXT("false")));
	}
	if (bPreferedMinimalPower)
	{
		UE_LOGF(LogStandaloneRenderer, Log, "bPreferedMinimalPower=%ls", (bPreferedMinimalPower ? TEXT("true") : TEXT("false")));
	}
	else if (bPreferedHighPerformance)
	{
		UE_LOGF(LogStandaloneRenderer, Log, "bPreferedHighPerformance=%ls", (bPreferedHighPerformance ? TEXT("true") : TEXT("false")));
	}
	if (bDebug)
	{
		UE_LOGF(LogStandaloneRenderer, Log, "bPreferedHighPerformance=%ls", (bPreferedHighPerformance ? TEXT("true") : TEXT("false")));
	}
}

HRESULT FFindBestDevice::EnumerateAdapters(UINT AdapterIndex, IDXGIAdapter** Adapter)
{
	if (!DXGIFactory6 || GpuPreference == DXGI_GPU_PREFERENCE_UNSPECIFIED)
	{
		return DXGIFactory1->EnumAdapters(AdapterIndex, Adapter);
	}
	else
	{
		return DXGIFactory6->EnumAdapterByGpuPreference(AdapterIndex, (DXGI_GPU_PREFERENCE)GpuPreference, __uuidof(IDXGIAdapter), (void**)Adapter);
	}
}

bool FFindBestDevice::CreateTesingDevice(IDXGIAdapter* Adapter, D3D_FEATURE_LEVEL& ResultFeatureLevel)
{
	ID3D11Device* D3DDevice = nullptr;
	ID3D11DeviceContext* D3DDeviceContext = nullptr;

	uint32 DeviceFlags = D3D11_CREATE_DEVICE_SINGLETHREADED;
	if (bDebug)
	{
		DeviceFlags |= D3D11_CREATE_DEVICE_DEBUG;
	}

	const D3D_FEATURE_LEVEL RequestedFeatureLevels[] = { D3D_FEATURE_LEVEL_11_1, D3D_FEATURE_LEVEL_11_0 };
	const int32 NumAllowedFeatureLevels = UE_ARRAY_COUNT(RequestedFeatureLevels);

	ResultFeatureLevel = D3D_FEATURE_LEVEL_1_0_CORE;
	__try
	{
		HRESULT CreateDeviceResult;
		CreateDeviceResult = D3D11CreateDevice(Adapter, D3D_DRIVER_TYPE_UNKNOWN, nullptr, DeviceFlags, RequestedFeatureLevels, NumAllowedFeatureLevels, D3D11_SDK_VERSION, &D3DDevice, &ResultFeatureLevel, &D3DDeviceContext);
		if (SUCCEEDED(CreateDeviceResult))
		{
			D3DDevice->Release();
			D3DDeviceContext->Release();
			return true;
		}
		else
		{
			UE_LOGF(LogStandaloneRenderer, Log, "Failed to create device [0x%08X]", CreateDeviceResult);
		}
	}
	__except (Private::IsDelayLoadException(GetExceptionInformation()))
	{
		// We suppress warning C6322: Empty _except block. Appropriate checks are made upon returning. 
		CA_SUPPRESS(6322);
	}

	return false;
}

void FFindBestDevice::SelectAdapter()
{
	TRefCountPtr<IDXGIAdapter> FirstDXGIAdapterWithoutIntegratedAdapter;
	TRefCountPtr<IDXGIAdapter> FirstDXGIAdapterAdapter;
	D3D_FEATURE_LEVEL FirstFeatureLevelWithoutIntegratedAdapter = D3D_FEATURE_LEVEL_1_0_CORE;
	D3D_FEATURE_LEVEL FirstFeatureLeveAdapter = D3D_FEATURE_LEVEL_1_0_CORE;

	UE_LOGF(LogStandaloneRenderer, Log, "D3D11 adapters:");

	// Enumerate the DXGIFactory's adapters.
	TRefCountPtr<IDXGIAdapter> TestingAdapter;
	for (uint32 AdapterIndex = 0; EnumerateAdapters(AdapterIndex, TestingAdapter.GetInitReference()) != DXGI_ERROR_NOT_FOUND; ++AdapterIndex)
	{
		DXGI_ADAPTER_DESC AdapterDesc;
		if (HRESULT DescResult = TestingAdapter->GetDesc(&AdapterDesc); FAILED(DescResult))
		{
			UE_LOGF(LogStandaloneRenderer, Warning, "Failed to get description for adapter %u.", AdapterIndex);
		}
		else
		{
			UE_LOGF(LogStandaloneRenderer, Log, "Testing D3D11 adapter: %u. Description: '%ls'. VendorId: %04x. DeviceId: %04x."
				, AdapterIndex, AdapterDesc.Description, AdapterDesc.VendorId, AdapterDesc.DeviceId);
		}

		D3D_FEATURE_LEVEL FeatureLevel;
		if (!CreateTesingDevice(TestingAdapter, FeatureLevel))
		{
			UE_LOGF(LogStandaloneRenderer, Log, "  Failed to create test device.");
			continue;
		}

		UE_LOGF(LogStandaloneRenderer, Log, "  %u. '%ls'. Feature level: %ls", AdapterIndex, AdapterDesc.Description, GetFeatureLevelString(FeatureLevel));

		const bool bIsMicrosoft = AdapterDesc.VendorId == 0x1414;
		const bool bIsSoftware = bIsMicrosoft;
		const bool bSkipSoftwareAdapter = bIsSoftware && !bAllowSoftwareRendering && ExplicitAdapterValue < 0;
		const bool bSkipExplicitAdapter = ExplicitAdapterValue >= 0 && AdapterIndex != ExplicitAdapterValue;
		const bool bSkipAdapter = bSkipSoftwareAdapter || bSkipExplicitAdapter;

		if (bSkipAdapter)
		{
			UE_LOGF(LogStandaloneRenderer, Log, "  Skip adapter.");
			continue;
		}


		bool bIsNonLocalMemoryPresent = false;
		{
			TRefCountPtr<IDXGIAdapter3> TempDxgiAdapter3;
			DXGI_QUERY_VIDEO_MEMORY_INFO NonLocalVideoMemoryInfo;
			if (SUCCEEDED(TestingAdapter->QueryInterface(_uuidof(IDXGIAdapter3), (void**)TempDxgiAdapter3.GetInitReference())) &&
				TempDxgiAdapter3.IsValid() && SUCCEEDED(TempDxgiAdapter3->QueryVideoMemoryInfo(0, DXGI_MEMORY_SEGMENT_GROUP_NON_LOCAL, &NonLocalVideoMemoryInfo)))
			{
				bIsNonLocalMemoryPresent = NonLocalVideoMemoryInfo.Budget != 0;
			}
		}
		const bool bIsIntegrated = !bIsNonLocalMemoryPresent;

		if (!bIsIntegrated && !FirstDXGIAdapterWithoutIntegratedAdapter.IsValid())
		{
			FirstDXGIAdapterWithoutIntegratedAdapter = TestingAdapter;
			FirstFeatureLevelWithoutIntegratedAdapter = FeatureLevel;
		}
		else if (PreferedAdapterVendor == AdapterDesc.VendorId && FirstDXGIAdapterWithoutIntegratedAdapter.IsValid())
		{
			FirstDXGIAdapterWithoutIntegratedAdapter = TestingAdapter;
			FirstFeatureLevelWithoutIntegratedAdapter = FeatureLevel;
		}

		if (!FirstDXGIAdapterAdapter.IsValid())
		{
			FirstDXGIAdapterAdapter = TestingAdapter;
			FirstFeatureLeveAdapter = FeatureLevel;
		}
		else if (PreferedAdapterVendor == AdapterDesc.VendorId && FirstDXGIAdapterAdapter.IsValid())
		{
			FirstDXGIAdapterAdapter = TestingAdapter;
			FirstFeatureLeveAdapter = FeatureLevel;
		}
	}

	const bool bFavorNonIntegrated = ExplicitAdapterValue == -1;
	if (bFavorNonIntegrated)
	{
		ChosenAdapter = FirstDXGIAdapterWithoutIntegratedAdapter;
		ChosenFeatureLevel = FirstFeatureLevelWithoutIntegratedAdapter;

		// We assume Intel is integrated graphics (slower than discrete) than NVIDIA or AMD cards and rather take a different one
		if (!ChosenAdapter.IsValid())
		{
			ChosenAdapter = FirstDXGIAdapterAdapter;
			ChosenFeatureLevel = FirstFeatureLeveAdapter;
		}
	}
	else
	{
		ChosenAdapter = FirstDXGIAdapterAdapter;
		ChosenFeatureLevel = FirstFeatureLeveAdapter;
	}

	if (ChosenAdapter.IsValid())
	{
		DXGI_ADAPTER_DESC AdapterDesc;
		HRESULT DescResult = ChosenAdapter->GetDesc(&AdapterDesc);
		if (FAILED(DescResult))
		{
			UE_LOGF(LogStandaloneRenderer, Warning, "Failed to get description for selected adapter.");
		}
		else
		{
			UE_LOGF(LogStandaloneRenderer, Log, "Selected D3D11 Description: '%ls'. VendorId: %04x. DeviceId: %04x."
				, AdapterDesc.Description, AdapterDesc.VendorId, AdapterDesc.DeviceId);
		}
	}
}

} //namespace
