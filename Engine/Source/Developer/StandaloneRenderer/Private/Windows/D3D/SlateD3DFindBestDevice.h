// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include "StandaloneRendererPlatformHeaders.h"

struct IDXGIAdapter;
struct IDXGIFactory1;
struct IDXGIFactory6;

namespace UE::StandaloneRenderer::D3D
{

struct FFindBestDevice
{
	FFindBestDevice();
	~FFindBestDevice();

	bool IsValid() const;

	bool CreateDevice(ID3D11Device** Device, ID3D11DeviceContext** DeviceContext);

	static const TCHAR* GetFeatureLevelString(D3D_FEATURE_LEVEL FeatureLevel);

private:
	void LoadSettings();
	void SelectAdapter();

	HRESULT EnumerateAdapters(UINT AdapterIndex, IDXGIAdapter** Adapter);
	bool CreateTesingDevice(IDXGIAdapter* Adapter, D3D_FEATURE_LEVEL& ResultFeatureLevel);

private:
	TRefCountPtr<IDXGIAdapter> ChosenAdapter;
	TRefCountPtr<IDXGIFactory1> DXGIFactory1;
	TRefCountPtr<IDXGIFactory6> DXGIFactory6;
	D3D_FEATURE_LEVEL ChosenFeatureLevel;
	int32 GpuPreference; /*DXGI_GPU_PREFERENCE*/
	int32 ExplicitAdapterValue = -1;
	int32 PreferedAdapterVendor = -1;
	bool bAllowSoftwareRendering = false;
	bool bPreferedMinimalPower = false;
	bool bPreferedHighPerformance = false;
	bool bDebug = false;
};

}
