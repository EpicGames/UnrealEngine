// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/D3D/SlateD3DIndexBuffer.h"
#include "Windows/D3D/SlateD3DRenderer.h"

FSlateD3DIndexBuffer::FSlateD3DIndexBuffer()
	: MaxNumIndices(0)	 
{

}

FSlateD3DIndexBuffer::~FSlateD3DIndexBuffer()
{

}

/** Initializes the index buffers RHI resource. */
void FSlateD3DIndexBuffer::CreateBuffer()
{
	if( MaxNumIndices == 0 )
	{
		MaxNumIndices = 1000;
	}

	uint32 SizeBytes = MaxNumIndices * sizeof(SlateIndex);

	D3D11_BUFFER_DESC BufferDesc;
	BufferDesc.Usage = D3D11_USAGE_DYNAMIC;
	BufferDesc.ByteWidth = SizeBytes;
	BufferDesc.BindFlags = D3D11_BIND_INDEX_BUFFER;
	BufferDesc.CPUAccessFlags = D3D11_CPU_ACCESS_WRITE;
	BufferDesc.MiscFlags = 0;

	HRESULT Hr = GD3DDevice->CreateBuffer( &BufferDesc, NULL, Buffer.GetInitReference() ) ;
	if (!SUCCEEDED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DIndexBuffer::CreateBuffer() - ID3D11Device::CreateBuffer"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
	}
}

/** Resizes the buffer to the passed in size.  Preserves internal data */
void FSlateD3DIndexBuffer::ResizeBuffer( uint32 NumIndices )
{
	if( NumIndices > MaxNumIndices )
	{
		uint32 BufferSize = MaxNumIndices*sizeof(SlateIndex);
		uint8 *SavedIndices = NULL;

		{
			void *Indices = Lock(0);

			SavedIndices = new uint8[MaxNumIndices*sizeof(SlateIndex)];
			FMemory::Memcpy( SavedIndices, Indices, BufferSize );

			Unlock();
		}

		D3D11_BUFFER_DESC BufferDesc;
		BufferDesc.Usage = D3D11_USAGE_DYNAMIC;
		BufferDesc.ByteWidth = NumIndices*sizeof(SlateIndex);
		BufferDesc.BindFlags = D3D11_BIND_INDEX_BUFFER;
		BufferDesc.CPUAccessFlags = D3D11_CPU_ACCESS_WRITE;
		BufferDesc.MiscFlags = 0;

		HRESULT Hr = GD3DDevice->CreateBuffer( &BufferDesc, NULL, Buffer.GetInitReference() ) ;

		if (SUCCEEDED(Hr))
		{
			if (SavedIndices)
			{
				void *Indices = Lock(0);

				FMemory::Memcpy(Indices, SavedIndices, BufferSize);

				Unlock();
			}

			MaxNumIndices = NumIndices;
		}
		else
		{
			LogSlateD3DRendererFailure(TEXT("FSlateD3DIndexBuffer::ResizeBuffer() - ID3D11Device::CreateBuffer"), Hr);
			GEncounteredCriticalD3DDeviceError = true;
		}

		delete[] SavedIndices;
	}
}

/** Locks the index buffer, returning a pointer to the indices */
void* FSlateD3DIndexBuffer::Lock( uint32 FirstIndex )
{
	D3D11_MAPPED_SUBRESOURCE Resource;
	GD3DDeviceContext->Map( Buffer, 0, D3D11_MAP_WRITE_DISCARD, 0, &Resource );
	void* Data = Resource.pData;

	uint32 Offset = FirstIndex * sizeof(SlateIndex);
	return (void*)((uint8*)Data + Offset);
}

void FSlateD3DIndexBuffer::Unlock()
{
	GD3DDeviceContext->Unmap( Buffer, 0 );
}

/** Releases the index buffers RHI resource. */
void FSlateD3DIndexBuffer::DestroyBuffer()
{
	Buffer.SafeRelease();
}
