// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include "StandaloneRendererPlatformHeaders.h"

class FSlateD3DIndexBuffer 
{
public:
	FSlateD3DIndexBuffer();
	virtual ~FSlateD3DIndexBuffer();

	/** Initializes the index buffers resource. */
	virtual void CreateBuffer();

	/** Releases the index buffers resource. */
	virtual void DestroyBuffer();

	/** Returns the maximum number of indices that can be used by this buffer */
	uint32 GetMaxNumIndices() const { return MaxNumIndices; }

	/** Resizes the buffer to the passed in size.  Preserves internal data */
	void ResizeBuffer( uint32 NumIndices );

	/** Locks the index buffer, returning a pointer to the indices */
	void* Lock( uint32 FirstIndex );

	/** Unlocks the index buffer */
	void Unlock();
	
	TRefCountPtr<ID3D11Buffer> GetResource() { return Buffer; }
private:
	uint32 MaxNumIndices;
	TRefCountPtr<ID3D11Buffer> Buffer;
	/** Hidden copy methods. */
	FSlateD3DIndexBuffer( const FSlateD3DIndexBuffer& );
	void operator=(const FSlateD3DIndexBuffer& );

};
