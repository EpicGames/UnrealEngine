// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/D3D/SlateD3DRenderer.h"
#include "Windows/D3D/SlateD3DTextureManager.h"
#include "Windows/D3D/SlateD3DRenderingPolicy.h"
#include "Windows/D3D/SlateD3DFindBestDevice.h"
#include "Rendering/ElementBatcher.h"
#include "Fonts/FontCache.h"
#include "Widgets/SWindow.h"
#include "Misc/CommandLine.h"
#include "StandaloneRendererLog.h"

TRefCountPtr<ID3D11Device> GD3DDevice;
TRefCountPtr<ID3D11DeviceContext> GD3DDeviceContext;
bool GEncounteredCriticalD3DDeviceError = false;

static FMatrix CreateProjectionMatrixD3D( uint32 Width, uint32 Height )
{
	// Create ortho projection matrix
	const float Left = 0.0f;//0.5f
	const float Right = Left+Width;
	const float Top = 0.0f;//0.5f
	const float Bottom = Top+Height;
	const float ZNear = 0;
	const float ZFar = 1;

	return	FMatrix( FPlane(2.0f/(Right-Left),			0,							0,					0 ),
					 FPlane(0,							2.0f/(Top-Bottom),			0,					0 ),
					 FPlane(0,							0,							1/(ZNear-ZFar),		0 ),
					 FPlane((Left+Right)/(Left-Right),	(Top+Bottom)/(Bottom-Top),	ZNear/(ZNear-ZFar), 1 ) );

}

FString GetReadableResult(HRESULT Hr)
{
	switch (Hr)
	{
#define CASE(A) case A: return TEXT(#A);
		CASE(DXGI_ERROR_DEVICE_HUNG)
		CASE(DXGI_ERROR_DEVICE_REMOVED)
		CASE(DXGI_ERROR_DEVICE_RESET)
		CASE(DXGI_ERROR_DRIVER_INTERNAL_ERROR)
		CASE(DXGI_ERROR_FRAME_STATISTICS_DISJOINT)
		CASE(DXGI_ERROR_GRAPHICS_VIDPN_SOURCE_IN_USE)
		CASE(DXGI_ERROR_INVALID_CALL)
		CASE(DXGI_ERROR_MORE_DATA)
		CASE(DXGI_ERROR_NONEXCLUSIVE)
		CASE(DXGI_ERROR_NOT_CURRENTLY_AVAILABLE)
		CASE(DXGI_ERROR_NOT_FOUND)
		CASE(DXGI_ERROR_REMOTE_CLIENT_DISCONNECTED)
		CASE(DXGI_ERROR_REMOTE_OUTOFMEMORY)
		CASE(DXGI_ERROR_WAS_STILL_DRAWING)
		CASE(DXGI_ERROR_UNSUPPORTED)
		CASE(DXGI_ERROR_ACCESS_LOST)
		CASE(DXGI_ERROR_WAIT_TIMEOUT)
		CASE(DXGI_ERROR_SESSION_DISCONNECTED)
		CASE(DXGI_ERROR_RESTRICT_TO_OUTPUT_STALE)
		CASE(DXGI_ERROR_CANNOT_PROTECT_CONTENT)
		CASE(DXGI_ERROR_ACCESS_DENIED)
		CASE(DXGI_ERROR_NAME_ALREADY_EXISTS)
		CASE(DXGI_ERROR_SDK_COMPONENT_MISSING)
#undef CASE
	}

	return FString::Printf(TEXT("DXGI_ERROR_%08X"), (int32)Hr);
}

void LogSlateD3DRendererFailure(const FString& Description, HRESULT Hr)
{
	UE_LOGF(LogStandaloneRenderer, Warning, "%ls Result: %ls [%X]", *Description, *GetReadableResult(Hr), Hr);

	if (Hr == DXGI_ERROR_DEVICE_REMOVED && IsValidRef(GD3DDevice))
	{
		HRESULT ReasonHr = GD3DDevice->GetDeviceRemovedReason();
		UE_LOGF(LogStandaloneRenderer, Warning, "%ls Reason: %ls [0x%08X]", *Description, *GetReadableResult(ReasonHr), ReasonHr);
	}
}

class FSlateD3DFontAtlasFactory : public ISlateFontAtlasFactory
{
public:

	virtual ~FSlateD3DFontAtlasFactory()
	{
	}

	virtual FIntPoint GetAtlasSize(ESlateFontAtlasContentType InContentType) const override
	{
		switch (InContentType)
		{
			case ESlateFontAtlasContentType::Alpha:
				return FIntPoint(GrayscaleTextureSize, GrayscaleTextureSize);
			case ESlateFontAtlasContentType::Color:
				return FIntPoint(ColorTextureSize, ColorTextureSize);
			case ESlateFontAtlasContentType::Msdf:
				return FIntPoint(SdfTextureSize, SdfTextureSize);
			default:
				checkNoEntry();
				// Default to Color
				return FIntPoint(ColorTextureSize, ColorTextureSize);
		}
	}

	virtual TSharedRef<FSlateFontAtlas> CreateFontAtlas(ESlateFontAtlasContentType InContentType) const override
	{
		const FIntPoint AtlasSize = GetAtlasSize(InContentType);
		return MakeShareable(new FSlateFontAtlasD3D(AtlasSize.X, AtlasSize.Y, InContentType));
	}

	virtual TSharedPtr<ISlateFontTexture> CreateNonAtlasedTexture(const uint32 InWidth, const uint32 InHeight, ESlateFontAtlasContentType InContentType, const TArray<uint8>& InRawData) const override
	{
		return nullptr;
	}

private:

	/** Size of each font texture, width and height */
	static const uint32 GrayscaleTextureSize = 1024;
	static const uint32 ColorTextureSize = 512;
	static const uint32 SdfTextureSize = 1024;
};

TSharedRef<FSlateFontServices> CreateD3DFontServices()
{
	const TSharedRef<FSlateFontCache> FontCache = MakeShareable(new FSlateFontCache(MakeShareable(new FSlateD3DFontAtlasFactory), ESlateTextureAtlasThreadId::Game));

	return MakeShareable(new FSlateFontServices(FontCache, FontCache));
}


FSlateD3DRenderer::FSlateD3DRenderer( const ISlateStyle& InStyle )
	: FSlateRenderer( CreateD3DFontServices() )
	, bHasAttemptedInitialization(false)
{

	ViewMatrix = FMatrix(	FPlane(1,	0,	0,	0),
							FPlane(0,	1,	0,	0),
							FPlane(0,	0,	1,  0),
							FPlane(0,	0,	0,	1));

}

FSlateD3DRenderer::~FSlateD3DRenderer()
{
}

bool FSlateD3DRenderer::Initialize()
{
	if (!bHasAttemptedInitialization)
	{
		bool bResult = CreateDevice();
		if (bResult)
		{
			GEncounteredCriticalD3DDeviceError = false;

			TextureManager = MakeShareable(new FSlateD3DTextureManager);

			if (!GEncounteredCriticalD3DDeviceError)
			{
				TextureManager->LoadUsedTextures();
			}

			if (!GEncounteredCriticalD3DDeviceError)
			{
				RenderingPolicy = MakeShareable(new FSlateD3D11RenderingPolicy(SlateFontServices.ToSharedRef(), TextureManager.ToSharedRef()));
			}

			if (!GEncounteredCriticalD3DDeviceError)
			{
				ElementBatcher = MakeUnique<FSlateElementBatcher>(RenderingPolicy.ToSharedRef());
			}
		}
		else
		{
			GEncounteredCriticalD3DDeviceError = true;
		}
	}

	bHasAttemptedInitialization = true;
	return bHasAttemptedInitialization && !GEncounteredCriticalD3DDeviceError;
}

void FSlateD3DRenderer::Destroy()
{
	FSlateShaderParameterMap::Get().Shutdown();
	ElementBatcher.Reset();
	RenderingPolicy.Reset();
	TextureManager.Reset();
	GD3DDevice.SafeRelease();
	GD3DDeviceContext.SafeRelease();
}

bool FSlateD3DRenderer::CreateDevice()
{
	bool bResult = true;

	if( !IsValidRef( GD3DDevice ) || !IsValidRef( GD3DDeviceContext ) )
	{
		UE::StandaloneRenderer::D3D::FFindBestDevice BestDeviceFinder;
		if (BestDeviceFinder.IsValid())
		{
			bResult = BestDeviceFinder.CreateDevice(GD3DDevice.GetInitReference(), GD3DDeviceContext.GetInitReference());
			if (!bResult)
			{
				GEncounteredCriticalD3DDeviceError = true;
			}
		}
	}

	return bResult;
}

FSlateDrawBuffer& FSlateD3DRenderer::AcquireDrawBuffer()
{
	ensureMsgf(!DrawBuffer.IsLocked(), TEXT("The DrawBuffer is already locked. Make sure to call ReleaseDrawBuffer to release the DrawBuffer"));
	DrawBuffer.Lock();

	// Clear out the buffer each time its accessed
	DrawBuffer.ClearBuffer();

	return DrawBuffer;
}

void FSlateD3DRenderer::ReleaseDrawBuffer(FSlateDrawBuffer& InWindowDrawBuffer)
{
	ensureMsgf(&DrawBuffer == &InWindowDrawBuffer, TEXT("It release a DrawBuffer that is not a member of the SlateD3DRenderer"));
	InWindowDrawBuffer.Unlock();
}

void FSlateD3DRenderer::LoadStyleResources( const ISlateStyle& Style )
{
	if ( TextureManager.IsValid() )
	{
		TextureManager->LoadStyleResources( Style );
	}
}

FSlateUpdatableTexture* FSlateD3DRenderer::CreateUpdatableTexture(uint32 Width, uint32 Height)
{
	FSlateD3DTexture* NewTexture = new FSlateD3DTexture(Width, Height);
	NewTexture->Init(DXGI_FORMAT_B8G8R8A8_UNORM, NULL, true, true);
	return NewTexture;
}

FSlateUpdatableTexture* FSlateD3DRenderer::CreateSharedHandleTexture(void* SharedHandle)
{
	FSlateD3DTexture* NewTexture = new FSlateD3DTexture();
	NewTexture->Init(SharedHandle);
	return NewTexture;
}


void FSlateD3DRenderer::ReleaseUpdatableTexture(FSlateUpdatableTexture* Texture)
{
	Texture->Cleanup();
}

ISlateAtlasProvider* FSlateD3DRenderer::GetTextureAtlasProvider()
{
	if( TextureManager.IsValid() )
	{
		return TextureManager->GetTextureAtlasProvider();
	}

	return nullptr;
}

FCriticalSection* FSlateD3DRenderer::GetResourceCriticalSection()
{
	return &ResourceCriticalSection;
}

void FSlateD3DRenderer::RegisterCurrentScene_Impl(FSceneInterface* Scene)
{
	// This is a no-op
}

int32 FSlateD3DRenderer::GetCurrentSceneIndex() const
{
	// This is a no-op
	return -1;
}

void FSlateD3DRenderer::SetCurrentSceneIndex(int32 InIndex)
{
	// This is a no-op
}

void FSlateD3DRenderer::ClearScenes() 
{
	// This is a no-op
}


void FSlateD3DRenderer::Private_CreateViewport( TSharedRef<SWindow> InWindow, const FVector2D &WindowSize )
{
	TSharedRef< FGenericWindow > NativeWindow = InWindow->GetNativeWindow().ToSharedRef();

	bool bFullscreen = IsViewportFullscreen( *InWindow );
	bool bWindowed = true;//@todo implement fullscreen: !bFullscreen;

	DXGI_SWAP_CHAIN_DESC SwapChainDesc;
	FMemory::Memzero(&SwapChainDesc, sizeof(SwapChainDesc) );
	SwapChainDesc.BufferCount = 1;
	SwapChainDesc.BufferDesc.Width = FMath::TruncToInt(WindowSize.X);
	SwapChainDesc.BufferDesc.Height = FMath::TruncToInt(WindowSize.Y);
	SwapChainDesc.BufferDesc.Format = DXGI_FORMAT_B8G8R8A8_UNORM;
	SwapChainDesc.BufferDesc.RefreshRate.Numerator = 0;
	SwapChainDesc.BufferDesc.RefreshRate.Denominator = 1;
	SwapChainDesc.BufferUsage = DXGI_USAGE_RENDER_TARGET_OUTPUT;
	SwapChainDesc.OutputWindow = (HWND)NativeWindow->GetOSWindowHandle();
	SwapChainDesc.SampleDesc.Count = 1;
	SwapChainDesc.SampleDesc.Quality = 0;
	SwapChainDesc.Windowed = bWindowed;
	SwapChainDesc.SwapEffect = DXGI_SWAP_EFFECT_DISCARD;
	SwapChainDesc.Flags = DXGI_SWAP_CHAIN_FLAG_ALLOW_MODE_SWITCH;

	TRefCountPtr<IDXGIDevice1> DXGIDevice;
	HRESULT Hr = GD3DDevice->QueryInterface( __uuidof(IDXGIDevice1), (void**)DXGIDevice.GetInitReference() );
	if (!SUCCEEDED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DRenderer::Private_CreateViewport() - ID3D11Device::QueryInterface"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	TRefCountPtr<IDXGIAdapter1> DXGIAdapter;
	Hr = DXGIDevice->GetParent(__uuidof(IDXGIAdapter1), (void **)DXGIAdapter.GetInitReference());
	if (!SUCCEEDED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DRenderer::Private_CreateViewport() - IDXGIDevice1::GetParent(IDXGIAdapter1)"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	TRefCountPtr<IDXGIFactory1> DXGIFactory;
	Hr = DXGIAdapter->GetParent(__uuidof(IDXGIFactory1), (void **)DXGIFactory.GetInitReference());
	if (!SUCCEEDED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DRenderer::Private_CreateViewport() - IDXGIAdapter1::GetParent(IDXGIFactory1)"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	FSlateD3DViewport Viewport;

	Hr = DXGIFactory->CreateSwapChain(DXGIDevice.GetReference(), &SwapChainDesc, Viewport.D3DSwapChain.GetInitReference());
	if (!SUCCEEDED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DRenderer::Private_CreateViewport() - IDXGIFactory1::CreateSwapChain"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	Hr = DXGIFactory->MakeWindowAssociation((HWND)NativeWindow->GetOSWindowHandle(), DXGI_MWA_NO_ALT_ENTER);
	if (!SUCCEEDED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DRenderer::Private_CreateViewport() - IDXGIFactory1::MakeWindowAssociation"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	uint32 Width = FMath::TruncToInt(WindowSize.X);
	uint32 Height = FMath::TruncToInt(WindowSize.Y);

	Viewport.ViewportInfo.MaxDepth = 1.0f;
	Viewport.ViewportInfo.MinDepth = 0.0f;
	Viewport.ViewportInfo.Width = Width;
	Viewport.ViewportInfo.Height = Height;
	Viewport.ViewportInfo.TopLeftX = 0;
	Viewport.ViewportInfo.TopLeftY = 0;

	CreateBackBufferResources(Viewport.D3DSwapChain, Viewport.BackBufferTexture, Viewport.RenderTargetView);

	if (Width > 0 && Height > 0)
	{
		Viewport.ProjectionMatrix = CreateProjectionMatrixD3D(Width, Height);
	}
	else
	{
		Viewport.ProjectionMatrix.SetIdentity();
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	ensure(!Viewport.ProjectionMatrix.ContainsNaN());
#endif

	WindowToViewportMap.Add(&InWindow.Get(), Viewport);
}

void FSlateD3DRenderer::RequestResize( const TSharedPtr<SWindow>& InWindow, uint32 NewSizeX, uint32 NewSizeY )
{
	const bool bFullscreen = IsViewportFullscreen( *InWindow );
	Private_ResizeViewport( InWindow.ToSharedRef(), NewSizeX, NewSizeY, bFullscreen );
}

void FSlateD3DRenderer::UpdateFullscreenState( const TSharedRef<SWindow> InWindow, uint32 OverrideResX, uint32 OverrideResY )
{
	//@todo not implemented
}

void FSlateD3DRenderer::ReleaseDynamicResource( const FSlateBrush& Brush )
{
	if (TextureManager.IsValid())
	{
		TextureManager->ReleaseDynamicTextureResource( Brush );
	}
}

bool FSlateD3DRenderer::GenerateDynamicImageResource(FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& Bytes)
{
	FSlateShaderResourceProxy* Result = nullptr;

	if (TextureManager.IsValid())
	{
		Result = TextureManager->CreateDynamicTextureResource(ResourceName, Width, Height, Bytes);
	}

	return Result != nullptr;
}

bool FSlateD3DRenderer::GenerateDynamicImageResource(FName ResourceName, FSlateTextureDataRef TextureData)
{
	return GenerateDynamicImageResource(ResourceName, TextureData->GetWidth(), TextureData->GetHeight(), TextureData->GetRawBytes());
}

FSlateResourceHandle FSlateD3DRenderer::GetResourceHandle(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale)
{
	if (!TextureManager.IsValid())
	{
		return FSlateResourceHandle();
	}

	return TextureManager->GetResourceHandle(Brush, LocalSize, DrawScale);
}

void FSlateD3DRenderer::RemoveDynamicBrushResource( TSharedPtr<FSlateDynamicImageBrush> BrushToRemove )
{
	DynamicBrushesToRemove.Add( BrushToRemove );
}

void FSlateD3DRenderer::Private_ResizeViewport( const TSharedRef<SWindow> InWindow, uint32 Width, uint32 Height, bool bFullscreen )
{
	FSlateD3DViewport* Viewport = WindowToViewportMap.Find( &InWindow.Get() );

	if( Viewport && ( Width != Viewport->ViewportInfo.Width || Height != Viewport->ViewportInfo.Height || bFullscreen != Viewport->bFullscreen ) )
	{
		GD3DDeviceContext->OMSetRenderTargets(0,NULL,NULL);

		Viewport->BackBufferTexture.SafeRelease();
		Viewport->RenderTargetView.SafeRelease();
		Viewport->DepthStencilView.SafeRelease();

		Viewport->ViewportInfo.Width = Width;
		Viewport->ViewportInfo.Height = Height;
		Viewport->bFullscreen = bFullscreen;

		if (Width > 0 && Height > 0)
		{
			Viewport->ProjectionMatrix = CreateProjectionMatrixD3D( Width, Height );
		}
		else
		{
			Viewport->ProjectionMatrix.SetIdentity();
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		ensure(!Viewport->ProjectionMatrix.ContainsNaN());
#endif

		DXGI_SWAP_CHAIN_DESC Desc;
		HRESULT Hr = Viewport->D3DSwapChain->GetDesc( &Desc );
		if (FAILED(Hr))
		{
			LogSlateD3DRendererFailure(TEXT("FSlateD3DRenderer::Private_ResizeViewport() - FSlateD3DViewport::IDXGISwapChain::GetDesc"), Hr);
			GEncounteredCriticalD3DDeviceError = true;
			return;
		}

		Hr = Viewport->D3DSwapChain->ResizeBuffers(Desc.BufferCount, Viewport->ViewportInfo.Width, Viewport->ViewportInfo.Height, Desc.BufferDesc.Format, Desc.Flags);
		if (FAILED(Hr))
		{
			LogSlateD3DRendererFailure(TEXT("FSlateD3DRenderer::Private_ResizeViewport() - FSlateD3DViewport::IDXGISwapChain::ResizeBuffers"), Hr);
			GEncounteredCriticalD3DDeviceError = true;
			return;
		}

		CreateBackBufferResources(Viewport->D3DSwapChain, Viewport->BackBufferTexture, Viewport->RenderTargetView);
	}
}

void FSlateD3DRenderer::CreateBackBufferResources( TRefCountPtr<IDXGISwapChain>& InSwapChain, TRefCountPtr<ID3D11Texture2D>& OutBackBuffer, TRefCountPtr<ID3D11RenderTargetView>& OutRTV )
{
	InSwapChain->GetBuffer( 0, __uuidof(ID3D11Texture2D), (void**)OutBackBuffer.GetInitReference() );

	D3D11_RENDER_TARGET_VIEW_DESC RTVDesc;
	RTVDesc.Format = DXGI_FORMAT_B8G8R8A8_UNORM;
	RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE2D;
	RTVDesc.Texture2D.MipSlice = 0;
	HRESULT Hr = GD3DDevice->CreateRenderTargetView( OutBackBuffer, &RTVDesc, OutRTV.GetInitReference() );
	if (!SUCCEEDED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DRenderer::CreateBackBufferResources() - ID3D11Device::CreateRenderTargetView"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
	}
}

void FSlateD3DRenderer::CreateViewport( const TSharedRef<SWindow> InWindow )
{
#if UE_BUILD_DEBUG
	// Ensure a viewport for this window doesnt already exist
	FSlateD3DViewport* Viewport = WindowToViewportMap.Find( &InWindow.Get() );
	check(!Viewport);
#endif

	FVector2D WindowSize = InWindow->GetSizeInScreen();

	Private_CreateViewport( InWindow, WindowSize );
}

void FSlateD3DRenderer::CreateDepthStencilBuffer( FSlateD3DViewport& Viewport )
{
	TRefCountPtr<ID3D11Texture2D> DepthStencil;
	D3D11_TEXTURE2D_DESC DescDepth;

	DescDepth.Width = Viewport.ViewportInfo.Width;
	DescDepth.Height = Viewport.ViewportInfo.Height;
	DescDepth.MipLevels = 1;
	DescDepth.ArraySize = 1;
	DescDepth.Format = DXGI_FORMAT_D24_UNORM_S8_UINT;
	DescDepth.SampleDesc.Count = 1;
	DescDepth.SampleDesc.Quality = 0;
	DescDepth.Usage = D3D11_USAGE_DEFAULT;
	DescDepth.BindFlags = D3D11_BIND_DEPTH_STENCIL;
	DescDepth.CPUAccessFlags = 0;
	DescDepth.MiscFlags = 0;
	HRESULT Hr = GD3DDevice->CreateTexture2D( &DescDepth, NULL, DepthStencil.GetInitReference() );
	if (SUCCEEDED(Hr))
	{
		D3D11_DEPTH_STENCIL_VIEW_DESC DescDSV;
		DescDSV.Format = DXGI_FORMAT_D24_UNORM_S8_UINT;
		DescDSV.ViewDimension = D3D11_DSV_DIMENSION_TEXTURE2D;
		DescDSV.Texture2D.MipSlice = 0;
		DescDSV.Flags = 0;

		// Create the depth stencil view
		Hr = GD3DDevice->CreateDepthStencilView(DepthStencil, &DescDSV, Viewport.DepthStencilView.GetInitReference());

		if (!SUCCEEDED(Hr))
		{
			LogSlateD3DRendererFailure(TEXT("FSlateD3DRenderer::CreateDepthStencilBuffer() - ID3D11Device::CreateDepthStencilView"), Hr);
			GEncounteredCriticalD3DDeviceError = true;
		}
	}
	else
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DRenderer::CreateDepthStencilBuffer() - ID3D11Device::CreateTexture2D"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
	}
}

bool FSlateD3DRenderer::HasLostDevice() const
{
	return (bHasAttemptedInitialization && (GEncounteredCriticalD3DDeviceError || !IsValidRef(GD3DDevice) || FAILED(GD3DDevice->GetDeviceRemovedReason())));
}

void FSlateD3DRenderer::DrawWindows(FSlateDrawBuffer& InWindowDrawBuffer)
{
	if (HasLostDevice())
	{
		return;
	}

	const TSharedRef<FSlateFontCache> FontCache = SlateFontServices->GetFontCache();

	// Iterate through each element list and set up an RHI window for it if needed
	const TArray< TSharedRef<FSlateWindowElementList> >& WindowElementLists = InWindowDrawBuffer.GetWindowElementLists();
	for( int32 ListIndex = 0; ListIndex < WindowElementLists.Num(); ++ListIndex )
	{
		FSlateWindowElementList& ElementList = *WindowElementLists[ListIndex];

		if ( ElementList.GetRenderWindow() )
		{
			if (HasLostDevice())
			{
				break;
			}

			SWindow* WindowToDraw = ElementList.GetRenderWindow();

			TextureManager->UpdateCache();

			// Add all elements for this window to the element batcher
			ElementBatcher->AddElements(ElementList);

			// Update the font cache with new text before elements are batched
			FontCache->UpdateCache();
	

			FVector2D WindowSize = WindowToDraw->GetSizeInScreen();

			FSlateD3DViewport* Viewport = WindowToViewportMap.Find( WindowToDraw );
			check(Viewport);

			FSlateBatchData& BatchData = ElementList.GetBatchData();
			{

				RenderingPolicy->BuildRenderingBuffers(BatchData);
			}

			check(Viewport);
			GD3DDeviceContext->RSSetViewports(1, &Viewport->ViewportInfo );

			ID3D11RenderTargetView* RTV = Viewport->RenderTargetView;
			ID3D11DepthStencilView* DSV = Viewport->DepthStencilView;
			
#if ALPHA_BLENDED_WINDOWS
			if ( WindowToDraw->GetTransparencySupport() == EWindowTransparency::PerPixel )
			{
				GD3DDeviceContext->ClearRenderTargetView(RTV, &FLinearColor::Transparent.R);
			}
#endif
			GD3DDeviceContext->OMSetRenderTargets( 1, &RTV, NULL );

			if (HasLostDevice())
			{
				break;
			}
			else
			{
				RenderingPolicy->DrawElements(ViewMatrix * Viewport->ProjectionMatrix, BatchData.GetFirstRenderBatchIndex(), BatchData.GetRenderBatches());
			}

			GD3DDeviceContext->OMSetRenderTargets(0, NULL, NULL);

			const bool bUseVSync = false;
			HRESULT Hr = Viewport->D3DSwapChain->Present( bUseVSync ? 1 : 0, 0);
			if (FAILED(Hr))
			{
				LogSlateD3DRendererFailure(TEXT("FSlateD3DRenderer::DrawWindows() - IDXGISwapChain::Viewport->D3DSwapChain->Present( bUseVSync ? 1 : 0, 0)"), Hr);
				GEncounteredCriticalD3DDeviceError = true;
				break;
			}

			// All elements have been drawn.  Reset all cached data
			ElementBatcher->ResetBatches();
		}
	}

	// flush the cache if needed
	FontCache->ConditionalFlushCache();
	TextureManager->ConditionalFlushCache();

	// Safely release the references now that we are finished rendering with the dynamic brushes
	DynamicBrushesToRemove.Empty();
}

void FSlateD3DRenderer::OnWindowDestroyed( const TSharedRef<SWindow>& InWindow )
{
	WindowToViewportMap.Remove(&InWindow.Get());
}
