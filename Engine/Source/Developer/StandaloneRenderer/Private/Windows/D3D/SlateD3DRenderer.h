// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include "Rendering/SlateRenderer.h"
#include "Rendering/SlateDrawBuffer.h"
#include "StandaloneRendererPlatformHeaders.h"

class FSlateD3DTextureManager;
class FSlateD3D11RenderingPolicy;

struct FSlateD3DViewport
{
	FMatrix ProjectionMatrix;
	D3D11_VIEWPORT ViewportInfo;
	TRefCountPtr<IDXGISwapChain> D3DSwapChain;
	TRefCountPtr<ID3D11Texture2D> BackBufferTexture;
	TRefCountPtr<ID3D11RenderTargetView> RenderTargetView;
	TRefCountPtr<ID3D11DepthStencilView> DepthStencilView;
	bool bFullscreen;

	FSlateD3DViewport()
		: bFullscreen( false )
	{

	}

	~FSlateD3DViewport()
	{	
		BackBufferTexture.SafeRelease();
		RenderTargetView.SafeRelease();
		DepthStencilView.SafeRelease();
		D3DSwapChain.SafeRelease();
	}

};

extern TRefCountPtr<ID3D11Device> GD3DDevice;
extern TRefCountPtr<ID3D11DeviceContext> GD3DDeviceContext;
extern bool GEncounteredCriticalD3DDeviceError;

void LogSlateD3DRendererFailure(const FString& Description, HRESULT Hr);

class FSlateD3DRenderer : public FSlateRenderer
{
public:
	STANDALONERENDERER_API FSlateD3DRenderer( const ISlateStyle &InStyle );
	~FSlateD3DRenderer();

	/** FSlateRenderer Interface */
	virtual bool Initialize() override;
	virtual void Destroy() override;
	virtual FSlateDrawBuffer& AcquireDrawBuffer() override;
	virtual void ReleaseDrawBuffer( FSlateDrawBuffer& InWindowDrawBuffer ) override;
	virtual void DrawWindows( FSlateDrawBuffer& InWindowDrawBuffer ) override;
	virtual void OnWindowDestroyed( const TSharedRef<SWindow>& InWindow ) override;
	virtual void CreateViewport( const TSharedRef<SWindow> InWindow ) override;
	virtual void RequestResize( const TSharedPtr<SWindow>& InWindow, uint32 NewSizeX, uint32 NewSizeY ) override;
	virtual void UpdateFullscreenState( const TSharedRef<SWindow> InWindow, uint32 OverrideResX, uint32 OverrideResY ) override;
	virtual void SetSystemResolution(uint32 Width, uint32 Height) override {}
	virtual void RestoreSystemResolution(const TSharedRef<SWindow> InWindow) override {}
	virtual void ReleaseDynamicResource( const FSlateBrush& Brush ) override;
	virtual bool GenerateDynamicImageResource(FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& Bytes) override;
	virtual bool GenerateDynamicImageResource(FName ResourceName, FSlateTextureDataRef TextureData) override;
	virtual FSlateResourceHandle GetResourceHandle(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale) override;
	virtual void RemoveDynamicBrushResource( TSharedPtr<FSlateDynamicImageBrush> BrushToRemove ) override;
	virtual void LoadStyleResources( const ISlateStyle& Style ) override;
	virtual FSlateUpdatableTexture* CreateUpdatableTexture(uint32 Width, uint32 Height) override;
	virtual FSlateUpdatableTexture* CreateSharedHandleTexture(void* SharedHandle) override;
	virtual void ReleaseUpdatableTexture(FSlateUpdatableTexture* Texture) override;
	virtual ISlateAtlasProvider* GetTextureAtlasProvider() override;
	virtual FCriticalSection* GetResourceCriticalSection() override;
	virtual int32 GetCurrentSceneIndex() const override;
	virtual void SetCurrentSceneIndex(int32 InIndex) override;
	virtual void ClearScenes() override;

protected:
	virtual void RegisterCurrentScene_Impl(FSceneInterface* Scene) override;

public:
	
	bool CreateDevice();
	void CreateDepthStencilBuffer( FSlateD3DViewport& Viewport );

	virtual bool HasLostDevice() const override;

private:
	void Private_CreateViewport( TSharedRef<SWindow> InWindow, const FVector2D& WindowSize );
	void Private_ResizeViewport( const TSharedRef<SWindow> InWindow, uint32 Width, uint32 Height, bool bFullscreen );
	void CreateBackBufferResources( TRefCountPtr<IDXGISwapChain>& InSwapChain, TRefCountPtr<ID3D11Texture2D>& OutBackBuffer, TRefCountPtr<ID3D11RenderTargetView>& OutRTV );

private:

	bool bHasAttemptedInitialization;

	FMatrix ViewMatrix;
	TMap<const SWindow*, FSlateD3DViewport> WindowToViewportMap;
	FSlateDrawBuffer DrawBuffer;
	TUniquePtr<FSlateElementBatcher> ElementBatcher;
	TSharedPtr<FSlateD3DTextureManager> TextureManager;
	TSharedPtr<FSlateD3D11RenderingPolicy> RenderingPolicy;
	TArray<TSharedPtr<FSlateDynamicImageBrush>> DynamicBrushesToRemove;
	FCriticalSection ResourceCriticalSection;
};
