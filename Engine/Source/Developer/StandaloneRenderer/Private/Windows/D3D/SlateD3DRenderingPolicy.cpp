// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/D3D/SlateD3DRenderingPolicy.h"
#include "Windows/D3D/SlateD3DRenderer.h"
#include "Windows/D3D/SlateD3DTextureManager.h"
#include "Windows/D3D/SlateD3DTextures.h"
#include "Layout/Clipping.h"


/** Offset to apply to UVs to line up texels with pixels */
const float PixelCenterOffsetD3D11 = 0.0f;


static D3D11_PRIMITIVE_TOPOLOGY GetD3D11PrimitiveType( ESlateDrawPrimitive SlateType )
{
	switch( SlateType )
	{
	case ESlateDrawPrimitive::LineList:
		return D3D11_PRIMITIVE_TOPOLOGY_LINELIST;
	case ESlateDrawPrimitive::TriangleList:
	default:
		return D3D11_PRIMITIVE_TOPOLOGY_TRIANGLELIST;
	}

};

FSlateD3D11RenderingPolicy::FSlateD3D11RenderingPolicy( TSharedRef<FSlateFontServices> InSlateFontServices, TSharedRef<FSlateD3DTextureManager> InTexureManager )
	: FSlateRenderingPolicy( InSlateFontServices, PixelCenterOffsetD3D11 ) 
	, TextureManager( InTexureManager )
{
	InitResources();
}

FSlateD3D11RenderingPolicy::~FSlateD3D11RenderingPolicy()
{
	ReleaseResources();
}

void FSlateD3D11RenderingPolicy::InitResources()
{
	VertexShader = nullptr;
	PixelShader = nullptr;

	D3D11_SAMPLER_DESC SamplerDesc;
	FMemory::Memzero( &SamplerDesc, sizeof(D3D11_SAMPLER_DESC) );
	SamplerDesc.Filter = D3D11_FILTER_MIN_MAG_MIP_POINT;
	SamplerDesc.AddressU = D3D11_TEXTURE_ADDRESS_WRAP;
	SamplerDesc.AddressV = D3D11_TEXTURE_ADDRESS_WRAP;
	SamplerDesc.AddressW = D3D11_TEXTURE_ADDRESS_WRAP;
	SamplerDesc.MaxAnisotropy = 1;
	SamplerDesc.ComparisonFunc = D3D11_COMPARISON_NEVER;
	SamplerDesc.MaxLOD = D3D11_FLOAT32_MAX;
	SamplerDesc.MinLOD = 0;

	HRESULT Hr = GD3DDevice->CreateSamplerState( &SamplerDesc, PointSamplerState_Wrap.GetInitReference() );
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3D11RenderingPolicy::InitResources() - ID3D11Device::CreateSamplerState"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	SamplerDesc.Filter = D3D11_FILTER_MIN_MAG_LINEAR_MIP_POINT;
	Hr = GD3DDevice->CreateSamplerState( &SamplerDesc, BilinearSamplerState_Wrap.GetInitReference() );
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3D11RenderingPolicy::InitResources() - ID3D11Device::CreateSamplerState(BilinearSamplerState_Wrap)"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	SamplerDesc.AddressU = D3D11_TEXTURE_ADDRESS_CLAMP;
	SamplerDesc.AddressV = D3D11_TEXTURE_ADDRESS_CLAMP;
	SamplerDesc.AddressW = D3D11_TEXTURE_ADDRESS_CLAMP;

	Hr = GD3DDevice->CreateSamplerState( &SamplerDesc, BilinearSamplerState_Clamp.GetInitReference() );
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3D11RenderingPolicy::InitResources() - ID3D11Device::CreateSamplerState(BilinearSamplerState_Clamp)"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	SamplerDesc.Filter = D3D11_FILTER_MIN_MAG_MIP_POINT;
	Hr = GD3DDevice->CreateSamplerState( &SamplerDesc, PointSamplerState_Clamp.GetInitReference() );
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3D11RenderingPolicy::InitResources() - ID3D11Device::CreateSamplerState(PointSamplerState_Clamp)"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	WhiteTexture = TextureManager->CreateColorTexture( TEXT("DefaultWhite"), FColor::White )->Resource;

	VertexBuffer.CreateBuffer( sizeof(FSlateVertex) );
	IndexBuffer.CreateBuffer();

	VertexShader = new FSlateDefaultVS;
	PixelShader = new FSlateDefaultPS;

	D3D11_BLEND_DESC BlendDesc;
	BlendDesc.AlphaToCoverageEnable = false;
	BlendDesc.IndependentBlendEnable = false;
	BlendDesc.RenderTarget[0].BlendEnable = true;
	BlendDesc.RenderTarget[0].SrcBlend = D3D11_BLEND_SRC_ALPHA;
	BlendDesc.RenderTarget[0].DestBlend = D3D11_BLEND_INV_SRC_ALPHA;
	BlendDesc.RenderTarget[0].BlendOp = D3D11_BLEND_OP_ADD;
	BlendDesc.RenderTarget[0].SrcBlendAlpha = D3D11_BLEND_ONE;
	BlendDesc.RenderTarget[0].DestBlendAlpha = D3D11_BLEND_INV_SRC_ALPHA;
	BlendDesc.RenderTarget[0].BlendOpAlpha = D3D11_BLEND_OP_ADD;
	BlendDesc.RenderTarget[0].RenderTargetWriteMask = D3D11_COLOR_WRITE_ENABLE_ALL;

	Hr = GD3DDevice->CreateBlendState( &BlendDesc, AlphaBlendState.GetInitReference() );
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3D11RenderingPolicy::InitResources() - ID3D11Device::CreateBlendState( &BlendDesc, AlphaBlendState.GetInitReference() )"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	BlendDesc.RenderTarget[0].BlendEnable = false;
	Hr = GD3DDevice->CreateBlendState( &BlendDesc, NoBlendState.GetInitReference() );
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3D11RenderingPolicy::InitResources() - ID3D11Device::CreateBlendState( &BlendDesc, NoBlendState.GetInitReference() )"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	D3D11_RASTERIZER_DESC RasterDesc;
	RasterDesc.CullMode = D3D11_CULL_NONE;
	RasterDesc.FillMode = D3D11_FILL_SOLID;
	RasterDesc.FrontCounterClockwise = true;
	RasterDesc.DepthBias = 0;
	RasterDesc.DepthBiasClamp = 0;
	RasterDesc.SlopeScaledDepthBias = 0;
	RasterDesc.ScissorEnable = false;
	RasterDesc.MultisampleEnable = false;
	RasterDesc.AntialiasedLineEnable = false;

	Hr = GD3DDevice->CreateRasterizerState( &RasterDesc, NormalRasterState.GetInitReference() );
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3D11RenderingPolicy::InitResources() - ID3D11Device::CreateRasterizerState( &RasterDesc, NormalRasterState.GetInitReference() )"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	RasterDesc.ScissorEnable = true;
	Hr = GD3DDevice->CreateRasterizerState( &RasterDesc, ScissorRasterState.GetInitReference() );
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3D11RenderingPolicy::InitResources() - ID3D11Device::CreateRasterizerState( &RasterDesc, ScissorRasterState.GetInitReference())"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	RasterDesc.AntialiasedLineEnable = false;
	RasterDesc.ScissorEnable = false;
	RasterDesc.FillMode = D3D11_FILL_WIREFRAME;
	Hr = GD3DDevice->CreateRasterizerState( &RasterDesc, WireframeRasterState.GetInitReference() );
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3D11RenderingPolicy::InitResources() - ID3D11Device::CreateRasterizerState( &RasterDesc, WireframeRasterState.GetInitReference() )"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	D3D11_DEPTH_STENCIL_DESC DSDesc;

	// Depth test parameters
	DSDesc.DepthEnable = false;
	DSDesc.DepthWriteMask = D3D11_DEPTH_WRITE_MASK_ZERO;
	DSDesc.DepthFunc = D3D11_COMPARISON_LESS;

	// Stencil test parameters
	DSDesc.StencilEnable = false;
	DSDesc.StencilReadMask = 0xFF;
	DSDesc.StencilWriteMask = 0xFF;

	Hr = GD3DDevice->CreateDepthStencilState(&DSDesc, DSStateOff.GetInitReference());
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3D11RenderingPolicy::InitResources() - ID3D11Device::CreateDepthStencilState"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}
}

/** Releases RHI resources used by the element batcher */
void FSlateD3D11RenderingPolicy::ReleaseResources()
{
	VertexBuffer.DestroyBuffer();
	IndexBuffer.DestroyBuffer();

	if (VertexShader != nullptr)
	{
		delete VertexShader;
		VertexShader = nullptr;
	}

	if (PixelShader != nullptr)
	{
		delete PixelShader;
		PixelShader = nullptr;
	}
}

void FSlateD3D11RenderingPolicy::BuildRenderingBuffers( FSlateBatchData& InBatchData )
{	
	if (!VertexBuffer.GetResource().IsValid() ||
		!IndexBuffer.GetResource().IsValid())
	{
		return;
	}

	if( InBatchData.GetRenderBatches().Num() > 0 )
	{
		const FSlateVertexArray& FinalVertexData = InBatchData.GetFinalVertexData();
		const FSlateIndexArray& FinalIndexData = InBatchData.GetFinalIndexData();

		if (FinalVertexData.Num() > 0)
		{
			const uint32 NumVertices = FinalVertexData.Num();
	
			// resize if needed
			if( NumVertices*sizeof(FSlateVertex) > VertexBuffer.GetBufferSize() )
			{
				uint32 NumBytesNeeded = NumVertices*sizeof(FSlateVertex);
				VertexBuffer.ResizeBuffer( NumBytesNeeded + (NumVertices / 4) * sizeof(FSlateVertex) );
			}
		}

		if (FinalIndexData.Num() > 0)
		{
			const uint32 NumIndices = FinalIndexData.Num();

			// resize if needed
			if( NumIndices > IndexBuffer.GetMaxNumIndices() )
			{
				IndexBuffer.ResizeBuffer( NumIndices + (NumIndices / 4) );
			}
		}

		uint8* VerticesPtr = nullptr;
		uint8* IndicesPtr = nullptr;
		{
			VerticesPtr = (uint8*)VertexBuffer.Lock(0);
			IndicesPtr = (uint8*)IndexBuffer.Lock(0);
		}
	
		{
			FMemory::Memcpy(VerticesPtr, FinalVertexData.GetData(), FinalVertexData.Num() * sizeof(FSlateVertex));
			FMemory::Memcpy(IndicesPtr, FinalIndexData.GetData(), FinalIndexData.Num() * sizeof(SlateIndex));
		}

		{
			VertexBuffer.Unlock();
			IndexBuffer.Unlock();
		}
	}
}

void FSlateD3D11RenderingPolicy::DrawElements(const FMatrix& ViewProjectionMatrix, int32 FirstBatchIndex, const TArray<FSlateRenderBatch>& RenderBatches)
{
	if (!VertexBuffer.GetResource().IsValid() ||
		!IndexBuffer.GetResource().IsValid())
	{
		return;
	}

	VertexShader->BindShader();

	ID3D11Buffer* Buffer = VertexBuffer.GetResource();
	const uint32 Stride = sizeof(FSlateVertex);

#if SLATE_USE_32BIT_INDICES
#define D3D_INDEX_FORMAT DXGI_FORMAT_R32_UINT
#else
#define D3D_INDEX_FORMAT DXGI_FORMAT_R16_UINT
#endif

	
	GD3DDeviceContext->IASetIndexBuffer( IndexBuffer.GetResource(), D3D_INDEX_FORMAT, 0 );

	VertexShader->SetViewProjection( ViewProjectionMatrix );

	PixelShader->BindShader();

	const FSlateClippingState* LastClippingState = nullptr;

	int32 NextRenderBatchIndex = FirstBatchIndex;
	while(NextRenderBatchIndex != INDEX_NONE)
	{
		const FSlateRenderBatch& RenderBatch = RenderBatches[NextRenderBatchIndex];

		NextRenderBatchIndex = RenderBatch.NextBatchIndex;

		const FSlateShaderResource* ShaderResource = RenderBatch.GetShaderResource();
		
		const ESlateBatchDrawFlag DrawFlags = RenderBatch.GetDrawFlags();

		const ESlateDrawEffect DrawEffects = RenderBatch.GetDrawEffects();

		const FShaderParams& ShaderParams = RenderBatch.GetShaderParams();

		VertexShader->BindParameters();
		
		if( EnumHasAllFlags(DrawFlags, ESlateBatchDrawFlag::NoBlending) )
		{
			GD3DDeviceContext->OMSetBlendState( NoBlendState, 0, 0xFFFFFFFF );
		}
		else
		{
			GD3DDeviceContext->OMSetBlendState( AlphaBlendState, 0, 0xFFFFFFFF );
		}

		if( EnumHasAllFlags(DrawFlags, ESlateBatchDrawFlag::Wireframe) )
		{
			GD3DDeviceContext->RSSetState( WireframeRasterState );
		}

		if (EnumHasAllFlags(DrawFlags, ESlateBatchDrawFlag::NoGamma))
		{
			PixelShader->SetGammaValues(FVector2f(1.0f, 1.0f));
		}
		else
		{
			PixelShader->SetGammaValues(FVector2f(1, 1 / 2.2f));
		}

		const FSlateClippingState* ClippingState = RenderBatch.GetClippingState();
		if (ClippingState != LastClippingState)
		{
			LastClippingState = ClippingState;

			if (ClippingState)
			{
				const FSlateClippingState& ClipState = *ClippingState;
				if (ClipState.ScissorRect.IsSet())
				{
					const FSlateClippingZone& ScissorRect = ClipState.ScissorRect.GetValue();

					D3D11_RECT R;
					R.left = ScissorRect.TopLeft.X;
					R.top = ScissorRect.TopLeft.Y;
					R.right = ScissorRect.BottomRight.X;
					R.bottom = ScissorRect.BottomRight.Y;
					GD3DDeviceContext->RSSetScissorRects(1, &R);
					GD3DDeviceContext->RSSetState(ScissorRasterState);
				}
				else
				{
					// We don't support stencil clipping on the d3d rendering policy.
					GD3DDeviceContext->RSSetState(NormalRasterState);
				}
			}
			else
			{
				GD3DDeviceContext->RSSetState(NormalRasterState);
			}
		}

		PixelShader->SetShaderType( static_cast<uint8>(RenderBatch.GetShaderType()) );

		// Disable stenciling and depth testing by default 
		GD3DDeviceContext->OMSetDepthStencilState( DSStateOff, 0x00);

		if(ShaderResource)
		{
			TRefCountPtr<ID3D11SamplerState> SamplerState;
			if( EnumHasAllFlags(DrawFlags, ESlateBatchDrawFlag::TileU) || EnumHasAllFlags(DrawFlags, ESlateBatchDrawFlag::TileV) )
			{
				SamplerState = BilinearSamplerState_Wrap;
			}
			else
			{
				SamplerState = BilinearSamplerState_Clamp;
			}

			PixelShader->SetTexture( ((FSlateD3DTexture*)ShaderResource)->GetTypedResource(), SamplerState );
		}
		else
		{
			PixelShader->SetTexture( ((FSlateD3DTexture*)WhiteTexture)->GetTypedResource(), BilinearSamplerState_Clamp );
		}

		PixelShader->SetShaderParams(ShaderParams);
		PixelShader->SetDrawEffects( DrawEffects );

		PixelShader->BindParameters();

		const uint32 Offset = RenderBatch.VertexOffset * sizeof(FSlateVertex);

		GD3DDeviceContext->IASetPrimitiveTopology(GetD3D11PrimitiveType(RenderBatch.GetDrawPrimitiveType()));

		const uint32 IndexCount = RenderBatch.GetNumIndices();

		check(IndexCount > 0);

		GD3DDeviceContext->IASetVertexBuffers(0, 1, &Buffer, &Stride, &Offset);
		check(RenderBatch.GetIndexOffset() + RenderBatch.GetNumIndices() <= IndexBuffer.GetMaxNumIndices());
		GD3DDeviceContext->DrawIndexed(IndexCount, RenderBatch.GetIndexOffset(), 0);
	}

	// Reset the Raster state when finished, there are places that are making assumptions about the raster state
	// that need to be fixed.
	GD3DDeviceContext->RSSetState(NormalRasterState);
}

TSharedRef<FSlateShaderResourceManager> FSlateD3D11RenderingPolicy::GetResourceManager() const
{
	return TextureManager;
}

