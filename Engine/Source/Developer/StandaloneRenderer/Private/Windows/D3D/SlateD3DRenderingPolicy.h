// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingPolicy.h"
#include "SlateD3DVertexBuffer.h"
#include "SlateD3DIndexBuffer.h"
#include "SlateD3DTextureManager.h"
#include "SlateD3DShaders.h"
#include "StandaloneRendererPlatformHeaders.h"

class FSlateD3D11RenderingPolicy : public FSlateRenderingPolicy
{
public:
	FSlateD3D11RenderingPolicy( TSharedRef<FSlateFontServices> InSlateFontServices, TSharedRef<FSlateD3DTextureManager> InTextureManager );
	~FSlateD3D11RenderingPolicy();

	void BuildRenderingBuffers( FSlateBatchData& InBatchData );
	void DrawElements(const FMatrix& ViewProjectionMatrix, int32 FirstBatchIndex, const TArray<FSlateRenderBatch>& RenderBatches);

	virtual TSharedRef<FSlateShaderResourceManager> GetResourceManager() const override;
	virtual bool IsVertexColorInLinearSpace() const override { return false; }

private:
	void InitResources();
	void ReleaseResources();
private:
	FSlateD3DVertexBuffer VertexBuffer;
	FSlateD3DIndexBuffer IndexBuffer;
	FSlateDefaultVS* VertexShader;
	FSlateDefaultPS* PixelShader;
	FSlateShaderResource* WhiteTexture;
	TRefCountPtr<ID3D11RasterizerState> NormalRasterState;
	TRefCountPtr<ID3D11RasterizerState> WireframeRasterState;
	TRefCountPtr<ID3D11RasterizerState> ScissorRasterState;
	TRefCountPtr<ID3D11BlendState> AlphaBlendState;
	TRefCountPtr<ID3D11BlendState> NoBlendState;
	TRefCountPtr<ID3D11DepthStencilState> DSStateOff;
	TRefCountPtr<ID3D11SamplerState> PointSamplerState_Wrap;
	TRefCountPtr<ID3D11SamplerState> PointSamplerState_Clamp;
	TRefCountPtr<ID3D11SamplerState> BilinearSamplerState_Wrap;
	TRefCountPtr<ID3D11SamplerState> BilinearSamplerState_Clamp;
	TSharedRef<FSlateD3DTextureManager> TextureManager;
};
