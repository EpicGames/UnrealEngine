// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/D3D/SlateD3DShaders.h"
#include "Windows/D3D/SlateD3DRenderer.h"
#include "Windows/D3D/SlateD3DRenderingPolicy.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/App.h"

#define DEFINE_GUID_FOR_CURRENT_COMPILER(name, l, w1, w2, b1, b2, b3, b4, b5, b6, b7, b8) \
	static const GUID name = { l, w1, w2, { b1, b2,  b3,  b4,  b5,  b6,  b7,  b8 } }

DEFINE_GUID_FOR_CURRENT_COMPILER(IID_ID3D11ShaderReflectionForCurrentCompiler, 0x8d536ca1, 0x0cca, 0x4956, 0xa8, 0x37, 0x78, 0x69, 0x63, 0x75, 0x55, 0x84);

typedef HRESULT(WINAPI* pD3DReflect)
(__in_bcount(SrcDataSize) LPCVOID pSrcData,
	__in SIZE_T  SrcDataSize,
	__in  REFIID pInterface,
	__out void** ppReflector);


static HMODULE GetCompilerModule()
{
	// Override default compiler path to newer dll
	FString CompilerPath = FPaths::EngineDir();
#if PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
	CompilerPath.Append(TEXT("Binaries/ThirdParty/Windows/DirectX/arm64/d3dcompiler_47.dll"));
#else
	CompilerPath.Append(TEXT("Binaries/ThirdParty/Windows/DirectX/x64/d3dcompiler_47.dll"));
#endif
	static bool bHasCompiler = false;
	static HMODULE CompilerDLL = 0;

	if (bHasCompiler == false)
	{
		CompilerDLL = LoadLibrary(*CompilerPath);
	}

	if (CompilerDLL == NULL)
	{
		// load the system one as the last resort
		CompilerDLL = LoadLibrary(TEXT("d3dcompiler_47.dll"));
	}

	if (CompilerDLL == NULL)
	{
		LogSlateD3DRendererFailure(FString::Printf(TEXT("Critical error. Compiler DLL %s could not be found, and loading the system one failed"), *CompilerPath), E_FAIL);
	}
	return CompilerDLL;
}

// @return pointer to the D3DCompile function
static pD3DCompile GetD3DCompileFunc()
{
	static HMODULE CompilerDLL = GetCompilerModule();

	if (CompilerDLL)
	{
		return (pD3DCompile)(void*)GetProcAddress(CompilerDLL, "D3DCompile");
	}

	return nullptr;
}

// @return pointer to the D3DCompile function
static pD3DReflect GetD3DReflectFunc()
{
	static HMODULE CompilerDLL = GetCompilerModule();
	if (CompilerDLL)
	{
		return (pD3DReflect)(void*)GetProcAddress(CompilerDLL, "D3DReflect");
	}

	return nullptr;
}


class StandaloneD3DIncluder final : public ID3DInclude
{
	public:
		~StandaloneD3DIncluder() 
		{
		}

		STDMETHOD(Open)(D3D_INCLUDE_TYPE IncludeType, LPCSTR pFileName, LPCVOID pParentData, LPCVOID* ppData, uint32* pBytes) override
		{
			FString FileName(ANSI_TO_TCHAR(pFileName));
			FString IncludePath = FPaths::EngineDir();
			IncludePath.Append(TEXT("Shaders/StandaloneRenderer/D3D/"));
			IncludePath.Append(FileName);

			TArray<uint8>* ShaderFilePtr = new TArray<uint8>();
			FFileHelper::LoadFileToArray(*ShaderFilePtr, *IncludePath);

			*ppData = ShaderFilePtr->GetData();
			*pBytes = ShaderFilePtr->Num();

			IncludeMap.Add(*ppData, ShaderFilePtr);

			return S_OK;
		}

		STDMETHOD(Close)(LPCVOID pData) override
		{
			TArray<uint8>* ShaderFilePtr = IncludeMap.FindChecked(pData);
			IncludeMap.Remove(pData);
			delete ShaderFilePtr;

			return S_OK;
		}

		TMap<LPCVOID, TArray<uint8>*>   IncludeMap;
};

static bool CompileShader( const FString& Filename, const FString& EntryPoint, const FString& ShaderModel, TRefCountPtr<ID3DBlob>& OutBlob )
{
	uint32 ShaderFlags = D3DCOMPILE_ENABLE_STRICTNESS;
#if UE_BUILD_DEBUG
	ShaderFlags |= D3DCOMPILE_DEBUG;
#else
	ShaderFlags |= D3DCOMPILE_OPTIMIZATION_LEVEL3;
#endif

	pD3DCompile D3DCompilerFunc = GetD3DCompileFunc();
	if (D3DCompilerFunc == nullptr)
	{
		GEncounteredCriticalD3DDeviceError = true;
		return false;
	}

	StandaloneD3DIncluder Includer;

	TArray<uint8> ShaderFile;
	if(FFileHelper::LoadFileToArray(ShaderFile, *Filename))
	{
		TRefCountPtr<ID3DBlob> ErrorBlob;
		HRESULT Hr = D3DCompilerFunc(ShaderFile.GetData(), ShaderFile.Num(), NULL, NULL, &Includer, TCHAR_TO_ANSI(*EntryPoint), TCHAR_TO_ANSI(*ShaderModel), ShaderFlags, 0, OutBlob.GetInitReference(), ErrorBlob.GetInitReference());

		if (FAILED(Hr))
		{
			LogSlateD3DRendererFailure(TEXT("SlateD3DShaders::CompileShader() - D3DCompilerFunc"), Hr);
			GEncounteredCriticalD3DDeviceError = true;

			if (ErrorBlob.GetReference())
			{
				LogSlateD3DRendererFailure(ANSI_TO_TCHAR(ErrorBlob->GetBufferPointer()), Hr);
			}
			else
			{
				LogSlateD3DRendererFailure(TEXT("D3DCompilerFunc failed, no error text provided"), Hr);
			}

			return false;
		}
	}
	else
	{
		LogSlateD3DRendererFailure(FString::Printf(TEXT("Failed to compile shader.  %s could not be found "), *Filename), E_FAIL);
		GEncounteredCriticalD3DDeviceError = true;
		return false;
	}

	return true;
}

static void GetShaderBindings( TRefCountPtr<ID3D11ShaderReflection> Reflector, FSlateD3DShaderBindings& OutBindings )
{
	D3D11_SHADER_DESC ShaderDesc;
	Reflector->GetDesc( &ShaderDesc );

	for( uint32 I = 0; I < ShaderDesc.BoundResources; ++I )
	{
		D3D11_SHADER_INPUT_BIND_DESC Desc;
		Reflector->GetResourceBindingDesc( I, &Desc );

		FSlateD3DShaderParameter* Param = FSlateShaderParameterMap::Get().Find( Desc.Name );
		if( Param )
		{
			if( Desc.Type == D3D_SIT_TEXTURE )
			{
				OutBindings.ResourceViews.Add( (TSlateD3DTypedShaderParameter<ID3D11ShaderResourceView>*)Param );
			}
			else if(  Desc.Type == D3D_SIT_CBUFFER  )
			{
				OutBindings.ConstantBuffers.Add( (TSlateD3DTypedShaderParameter<ID3D11Buffer>*)Param );
			}
			else if( Desc.Type == D3D_SIT_SAMPLER )
			{
				OutBindings.SamplerStates.Add( (TSlateD3DTypedShaderParameter<ID3D11SamplerState>*)Param );
			}
			else
			{
				// unhandled param type
				check(0);
			}
		}
		else
		{
			// not registered
			check(0);
		}

	}
}

void FSlateD3DVS::Create( const FString& Filename, const FString& EntryPoint, const FString& ShaderModel, D3D11_INPUT_ELEMENT_DESC* VertexLayout, uint32 VertexLayoutCount )
{
	TRefCountPtr<ID3DBlob> Blob;
	if(CompileShader( Filename, EntryPoint, ShaderModel, Blob))
	{
		HRESULT Hr = GD3DDevice->CreateVertexShader(Blob->GetBufferPointer(), Blob->GetBufferSize(), NULL, VertexShader.GetInitReference());
		if (FAILED(Hr))
		{
			LogSlateD3DRendererFailure(TEXT("FSlateD3DVS::Create() - ID3D11Device::CreateVertexShader"), Hr);
			GEncounteredCriticalD3DDeviceError = true;
			return;
		}

		Hr = GD3DDevice->CreateInputLayout(VertexLayout, VertexLayoutCount, Blob->GetBufferPointer(), Blob->GetBufferSize(), InputLayout.GetInitReference());
		if (FAILED(Hr))
		{
			LogSlateD3DRendererFailure(TEXT("FSlateD3DVS::Create() - ID3D11Device::CreateInputLayout"), Hr);
			GEncounteredCriticalD3DDeviceError = true;
			return;
		}

		pD3DReflect D3DReflectFunc = GetD3DReflectFunc();
		if (D3DReflectFunc == nullptr)
		{
			GEncounteredCriticalD3DDeviceError = true;
			return;
		}

		TRefCountPtr<ID3D11ShaderReflection> Reflector;
		Hr = D3DReflectFunc(Blob->GetBufferPointer(), Blob->GetBufferSize(), IID_ID3D11ShaderReflectionForCurrentCompiler, (void**)Reflector.GetInitReference());
		if (FAILED(Hr))
		{
			LogSlateD3DRendererFailure(TEXT("FSlateD3DVS::Create() - D3DReflect"), Hr);
			GEncounteredCriticalD3DDeviceError = true;
			return;
		}

		GetShaderBindings(Reflector, ShaderBindings);
	}
	else
	{
		GEncounteredCriticalD3DDeviceError = true;
	}
}


void FSlateD3DVS::BindShader()
{
	GD3DDeviceContext->IASetInputLayout( InputLayout );
	GD3DDeviceContext->VSSetShader( VertexShader, NULL, 0 );
}

void FSlateD3DVS::BindParameters()
{
	UpdateParameters();

	int32 NumViews = ShaderBindings.ResourceViews.Num();
	if( NumViews > 0 )
	{
		ID3D11ShaderResourceView** const Views = new ID3D11ShaderResourceView*[ NumViews ];
		for( int32 I = 0; I < NumViews; ++I )
		{
			Views[I] = ShaderBindings.ResourceViews[I]->GetParameter().GetReference();
		}

		GD3DDeviceContext->VSSetShaderResources(0, NumViews, Views);

		delete[] Views;
	}

	if( ShaderBindings.ConstantBuffers.Num() > 0 )
	{
		const uint32 BufferCount = ShaderBindings.ConstantBuffers.Num();
		ID3D11Buffer** const ConstantBuffers = new ID3D11Buffer*[ BufferCount ];
		for( uint32 I = 0; I < BufferCount; ++I )
		{
			ConstantBuffers[I] = ShaderBindings.ConstantBuffers[I]->GetParameter().GetReference();
		}

		GD3DDeviceContext->VSSetConstantBuffers(0, ShaderBindings.ConstantBuffers.Num(), ConstantBuffers);

		delete[] ConstantBuffers;
	}	
}

void FSlateD3DPS::Create( const FString& Filename, const FString& EntryPoint, const FString& ShaderModel )
{
	TRefCountPtr<ID3DBlob> Blob;
	if(CompileShader( Filename, EntryPoint, ShaderModel, Blob))
	{
		HRESULT Hr = GD3DDevice->CreatePixelShader(Blob->GetBufferPointer(), Blob->GetBufferSize(), NULL, PixelShader.GetInitReference());
		if (FAILED(Hr))
		{
			LogSlateD3DRendererFailure(TEXT("FSlateD3DPS::Create() - ID3D11Device::CreatePixelShader"), Hr);
			GEncounteredCriticalD3DDeviceError = true;
			return;
		}

		pD3DReflect D3DReflectFunc = GetD3DReflectFunc();
		if (D3DReflectFunc == nullptr)
		{
			GEncounteredCriticalD3DDeviceError = true;
			return;
		}

		TRefCountPtr<ID3D11ShaderReflection> Reflector;
		Hr = D3DReflectFunc(Blob->GetBufferPointer(), Blob->GetBufferSize(), IID_ID3D11ShaderReflectionForCurrentCompiler, (void**)Reflector.GetInitReference());
		if (FAILED(Hr))
		{
			LogSlateD3DRendererFailure(TEXT("FSlateD3DPS::Create() - D3DReflect"), Hr);
			GEncounteredCriticalD3DDeviceError = true;
			return;
		}

		GetShaderBindings(Reflector, ShaderBindings);
	}
	else
	{
		GEncounteredCriticalD3DDeviceError = true;
	}
}


void FSlateD3DPS::BindShader()
{
	GD3DDeviceContext->PSSetShader( PixelShader, NULL, 0 );
}

void FSlateD3DPS::BindParameters()
{
	UpdateParameters();

	int32 NumViews = ShaderBindings.ResourceViews.Num();

	if( NumViews )
	{
		ID3D11ShaderResourceView** const Views = new ID3D11ShaderResourceView*[ NumViews ];
		for( int32 I = 0; I < NumViews; ++I )
		{
			Views[I] = ShaderBindings.ResourceViews[I]->GetParameter().GetReference();
		}

		GD3DDeviceContext->PSSetShaderResources(0, NumViews, Views);

		delete[] Views;
	}

	int32 NumBuffers = ShaderBindings.ConstantBuffers.Num();

	if( NumBuffers )
	{
		ID3D11Buffer** const ConstantBuffers = new ID3D11Buffer*[ NumBuffers ];
		for( int32 I = 0; I < NumBuffers; ++I )
		{
			ConstantBuffers[I] = ShaderBindings.ConstantBuffers[I]->GetParameter().GetReference();
		}

		GD3DDeviceContext->PSSetConstantBuffers(0, NumBuffers, ConstantBuffers);

		delete[] ConstantBuffers;
	}	

	if( ShaderBindings.SamplerStates.Num() )
	{
		const uint32 StateCount = ShaderBindings.SamplerStates.Num();
		ID3D11SamplerState** const SamplerStates = new ID3D11SamplerState*[ StateCount ];
		for( uint32 I = 0; I < StateCount; ++I )
		{
			SamplerStates[I] = ShaderBindings.SamplerStates[I]->GetParameter().GetReference();
		}

		GD3DDeviceContext->PSSetSamplers(0, ShaderBindings.SamplerStates.Num(), SamplerStates);

		delete[] SamplerStates;
	}	
}


FSlateDefaultVS::FSlateDefaultVS()
{
	Constants = &FSlateShaderParameterMap::Get().RegisterParameter<ID3D11Buffer>( "PerElementVSConstants" );

	ConstantBuffer.Create();

	D3D11_INPUT_ELEMENT_DESC Layout[] = 
	{
		{ "TEXCOORD",	0, DXGI_FORMAT_R32G32B32A32_FLOAT,	0, 0,								D3D11_INPUT_PER_VERTEX_DATA,	0 },
		{ "TEXCOORD",	1, DXGI_FORMAT_R32G32_FLOAT,		0, D3D11_APPEND_ALIGNED_ELEMENT,	D3D11_INPUT_PER_VERTEX_DATA,	0 },
		{ "POSITION",	0, DXGI_FORMAT_R32G32_FLOAT,		0, D3D11_APPEND_ALIGNED_ELEMENT,	D3D11_INPUT_PER_VERTEX_DATA,	0 },
		{ "COLOR",		0, DXGI_FORMAT_B8G8R8A8_UNORM,		0, D3D11_APPEND_ALIGNED_ELEMENT,	D3D11_INPUT_PER_VERTEX_DATA,	0 },
		{ "COLOR",		1, DXGI_FORMAT_B8G8R8A8_UNORM,		0, D3D11_APPEND_ALIGNED_ELEMENT,	D3D11_INPUT_PER_VERTEX_DATA,	0 },
	};

	Create( FString::Printf( TEXT("%s/StandaloneRenderer/D3D/SlateDefaultVertexShader.hlsl"), FPlatformProcess::ShaderDir() ), TEXT("Main"), TEXT("vs_4_0"), Layout, UE_ARRAY_COUNT(Layout) );
}

void FSlateDefaultVS::SetViewProjection( const FMatrix& ViewProjectionMatrix )
{
	ConstantBuffer.GetBufferData().ViewProjection = (FMatrix44f)ViewProjectionMatrix;	// LWC_TODO: Precision loss
}

void FSlateDefaultVS::SetShaderParams( const FVector4& InShaderParams )
{
	ConstantBuffer.GetBufferData().VertexShaderParams = FVector4f(InShaderParams);		// LWC_TODO: Precision loss
}

void FSlateDefaultVS::UpdateParameters()
{

	ConstantBuffer.UpdateBuffer();

	// Set the constant parameter to use our constant buffer
	Constants->SetParameter( ConstantBuffer.GetResource() );
}


FSlateDefaultPS::FSlateDefaultPS()
{
	Texture = &FSlateShaderParameterMap::Get().RegisterParameter<ID3D11ShaderResourceView>( "ElementTexture" );
	TextureSampler = &FSlateShaderParameterMap::Get().RegisterParameter<ID3D11SamplerState>( "ElementTextureSampler" );
	PerFrameCBufferParam = &FSlateShaderParameterMap::Get().RegisterParameter<ID3D11Buffer>("PerFramePSConstants");
	PerElementCBufferParam = &FSlateShaderParameterMap::Get().RegisterParameter<ID3D11Buffer>("PerElementPSConstants");

	PerFrameConstants.Create();
	PerElementConstants.Create();

	PerFrameConstants.GetBufferData().GammaValues = FVector2f(1, 1 / 2.2f);

	PerFrameCBufferParam->SetParameter(PerFrameConstants.GetResource());

	// Set the constant parameter to use our constant buffer
	// @todo: If we go back to multiple pixel shaders this likely has be called more frequently
	PerElementCBufferParam->SetParameter(PerElementConstants.GetResource());

	Create( FString::Printf( TEXT("%s/StandaloneRenderer/D3D/SlateElementPixelShader.hlsl"), FPlatformProcess::ShaderDir()), TEXT("Main"), TEXT("ps_4_0") );

}

void FSlateDefaultPS::SetShaderType( uint32 InShaderType )
{
	PerElementConstants.GetBufferData().ShaderType = InShaderType;
}

void FSlateDefaultPS::SetDrawEffects( ESlateDrawEffect InDrawEffects )
{
	PerElementConstants.GetBufferData().IgnoreTextureAlpha = (uint32)(InDrawEffects & ESlateDrawEffect::IgnoreTextureAlpha);
	PerElementConstants.GetBufferData().DisableEffect = (uint32)(InDrawEffects & ESlateDrawEffect::DisabledEffect);
}

void FSlateDefaultPS::SetShaderParams(const FShaderParams& InShaderParams)
{
	PerElementConstants.GetBufferData().ShaderParams = InShaderParams.PixelParams;
	PerElementConstants.GetBufferData().ShaderParams2 = InShaderParams.PixelParams2;
}

void FSlateDefaultPS::SetGammaValues(const FVector2f& InGammaValues)
{
	PerFrameConstants.GetBufferData().GammaValues = InGammaValues;
}

void FSlateDefaultPS::UpdateParameters()
{
	PerFrameConstants.UpdateBuffer();
	PerElementConstants.UpdateBuffer();

	TextureSampler->SetParameter( SamplerState );
}
