// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StandaloneRendererPlatformHeaders.h"
#include "SlateD3DConstantBuffer.h"
#include "Rendering/RenderingCommon.h"

class FSlateD3DShaderParameter
{
public:
	virtual ~FSlateD3DShaderParameter() {}
};

template <typename ParamType>
class TSlateD3DTypedShaderParameter : public FSlateD3DShaderParameter
{
public:
	void SetParameter( const TRefCountPtr<ParamType>& InParam) { Param = InParam; }
	const TRefCountPtr<ParamType>& GetParameter() const { return Param; }
private:
	TRefCountPtr<ParamType> Param;
};

class FSlateShaderParameterMap
{
public:
	static FSlateShaderParameterMap& Get()
	{
		static TSharedPtr<FSlateShaderParameterMap> Instance;
		if( !Instance.IsValid() ) 
		{
			Instance = MakeShareable( new FSlateShaderParameterMap );
		}

		return *Instance;
	}

	template <typename ParamType>
	TSlateD3DTypedShaderParameter<ParamType>& RegisterParameter( const FString& ParamName )
	{
		check( ParamName.Len() > 0 );
		
		FSlateD3DShaderParameter** Param = NameToParameterMap.Find( ParamName );
		if( !Param )
		{
			TSlateD3DTypedShaderParameter<ParamType>* NewParam = new TSlateD3DTypedShaderParameter<ParamType>;
			NameToParameterMap.Add( ParamName, NewParam );
			return *NewParam;
		}
		else
		{
			return **(TSlateD3DTypedShaderParameter<ParamType>**)Param;
		}
	}

	FSlateD3DShaderParameter* Find( const FString& ParamName )
	{
		check( ParamName.Len() > 0 );
		FSlateD3DShaderParameter** Param = NameToParameterMap.Find( ParamName );
		if( !Param )
		{
			return NULL;
		}
		else
		{
			return *Param;
		}
	}

	void Shutdown()
	{
		for( TMap<FString,FSlateD3DShaderParameter*>::TIterator It(NameToParameterMap); It; ++It)
		{
			delete It.Value();
		}

		NameToParameterMap.Empty();
	}

	~FSlateShaderParameterMap()
	{
		Shutdown();
	}

private:
	FSlateShaderParameterMap() {}
private:
	TMap<FString, FSlateD3DShaderParameter*> NameToParameterMap;
};



struct FSlateD3DShaderBindings
{
	TArray< TSlateD3DTypedShaderParameter<ID3D11ShaderResourceView>* > ResourceViews;
	TArray< TSlateD3DTypedShaderParameter<ID3D11SamplerState>* > SamplerStates;
	TArray< TSlateD3DTypedShaderParameter<ID3D11Buffer>* > ConstantBuffers;
};

class FSlateD3DVS
{
public:
	virtual ~FSlateD3DVS() {}
	void Create( const FString& Filename, const FString& EntryPoint, const FString& ShaderModel, D3D11_INPUT_ELEMENT_DESC* VertexLayout, uint32 VertexLayoutCount );
	void BindShader();
	void BindParameters();
protected:
	virtual void UpdateParameters() {}
private:
	FSlateD3DShaderBindings ShaderBindings;
	TRefCountPtr<ID3D11VertexShader> VertexShader;
	TRefCountPtr<ID3D11InputLayout> InputLayout;
};


class FSlateD3DPS
{
public:
	virtual ~FSlateD3DPS() {}
	void Create( const FString& Filename, const FString& EntryPoint, const FString& ShaderModel );
	void BindShader();
	void BindParameters();
protected:
	virtual void UpdateParameters() {}
private:
	FSlateD3DShaderBindings ShaderBindings;
	TRefCountPtr<ID3D11PixelShader> PixelShader;
};

class FSlateDefaultVS : public FSlateD3DVS
{
public:
	FSlateDefaultVS();
	void SetViewProjection( const FMatrix& ViewProjectionMatrix );
	void SetShaderParams( const FVector4& InShaderParams );
protected:
	void UpdateParameters();
private:
	
	struct FPerElementConstants
	{
		FMatrix44f ViewProjection;
		FVector4f VertexShaderParams;
	};
	static_assert(sizeof(FPerElementConstants) == sizeof(float) * 20, "Size mismatch");	// LWC_TODO: Fix for FVector4f

	FSlateD3DConstantBuffer<FPerElementConstants> ConstantBuffer;
	TSlateD3DTypedShaderParameter<ID3D11Buffer>* Constants;
};

class FSlateDefaultPS : public FSlateD3DPS
{
public:
	FSlateDefaultPS();

	void SetTexture( const TRefCountPtr<ID3D11ShaderResourceView>& InTexture, TRefCountPtr<ID3D11SamplerState>& InSamplerState )
	{
		Texture->SetParameter( InTexture );
		SamplerState = InSamplerState;
	}

	void SetShaderType(uint32 InShaderType);
	void SetDrawEffects(ESlateDrawEffect InDrawEffects);
	void SetShaderParams(const FShaderParams& InShaderParams);
	void SetGammaValues(const FVector2f& InGammaValues);
protected:
	void UpdateParameters();
private:
	MS_ALIGN(16) struct FPerElementConstants
	{
		FVector4f ShaderParams;		// 16 bytes
		FVector4f ShaderParams2;	// 16 bytes
		uint32 ShaderType;			//  4 bytes
		uint32 IgnoreTextureAlpha;	//	4 bytes
		uint32 DisableEffect;		//  4 bytes
		uint32 UNUSED[1];			//  4 bytes
	};
	static_assert(sizeof(FPerElementConstants) == sizeof(float) * 8 + sizeof(uint32) * 4, "Size mismatch");	// LWC_TODO: Fix for FVector4f

	MS_ALIGN(16) struct FPerFrameConstants
	{
		FVector2f GammaValues;
	};
	static_assert(sizeof(FVector2f) == sizeof(float) * 2, "Size mismatch");

	FSlateD3DConstantBuffer<FPerElementConstants> PerElementConstants;
	FSlateD3DConstantBuffer<FPerFrameConstants> PerFrameConstants;
	TSlateD3DTypedShaderParameter<ID3D11ShaderResourceView>* Texture;
	TSlateD3DTypedShaderParameter<ID3D11SamplerState>* TextureSampler;
	TSlateD3DTypedShaderParameter<ID3D11Buffer>* PerFrameCBufferParam;
	TSlateD3DTypedShaderParameter<ID3D11Buffer>* PerElementCBufferParam;
	TRefCountPtr<ID3D11SamplerState> SamplerState;
};

