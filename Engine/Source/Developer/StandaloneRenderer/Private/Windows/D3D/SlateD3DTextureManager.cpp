// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/D3D/SlateD3DTextureManager.h"
#include "Windows/D3D/SlateD3DRenderer.h"

#include "StandaloneRendererPrivate.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Rendering/SlateVectorGraphicsCache.h"
#include "Textures/SlateShaderResource.h"


DEFINE_LOG_CATEGORY_STATIC(LogSlateD3D, Log, All);

class FSlateD3DTextureAtlasFactory : public ISlateTextureAtlasFactory
{
public:
	virtual TUniquePtr<FSlateTextureAtlas> CreateTextureAtlas(int32 AtlasSize, int32 AtlasStride, ESlateTextureAtlasPaddingStyle PaddingStyle, bool bUpdatesAfterInitialization) const
	{
		return CreateTextureAtlasInternal(AtlasSize, AtlasStride, PaddingStyle, bUpdatesAfterInitialization);
	}

	virtual TUniquePtr<FSlateShaderResource> CreateNonAtlasedTexture(const uint32 InWidth, const uint32 InHeight, const TArray<uint8>& InRawData) const
	{
		// Keep track of non-atlased textures so we can free their resources later
		TUniquePtr<FSlateD3DTexture> Texture = MakeUnique<FSlateD3DTexture>(InWidth, InHeight);

		D3D11_SUBRESOURCE_DATA InitData;
		InitData.pSysMem = InRawData.GetData();
		InitData.SysMemPitch = InWidth * 4;

		Texture->Init(DXGI_FORMAT_B8G8R8A8_UNORM_SRGB, &InitData);

		return Texture;
	}

	virtual void ReleaseTextureAtlases(const TArray<TUniquePtr<FSlateTextureAtlas>>& InTextureAtlases, const TArray<TUniquePtr<FSlateShaderResource>>& InNonAtlasedTextures, const bool bWaitForRelease) const
	{
		// nothing to do
	}

	static TUniquePtr<FSlateTextureAtlasD3D> CreateTextureAtlasInternal(int32 AtlasSize, int32 AtlasStride, ESlateTextureAtlasPaddingStyle PaddingStyle, bool bUpdatesAfterInitialization)
	{
		return MakeUnique<FSlateTextureAtlasD3D>(AtlasSize, AtlasSize, AtlasStride, PaddingStyle);
	}
};

FSlateD3DTextureManager::FDynamicTextureResource::FDynamicTextureResource( FSlateD3DTexture* ExistingTexture )
	: Proxy( new FSlateShaderResourceProxy )
	, D3DTexture( ExistingTexture  )
{
}

FSlateD3DTextureManager::FDynamicTextureResource::~FDynamicTextureResource()
{
	if( Proxy )
	{
		delete Proxy;
	}

	if( D3DTexture )
	{
		delete D3DTexture;
	}
}

FSlateD3DTextureManager::FSlateD3DTextureManager()
{
	VectorGraphicsCache = MakeUnique<FSlateVectorGraphicsCache>(MakeShared<FSlateD3DTextureAtlasFactory>());
}

FSlateD3DTextureManager::~FSlateD3DTextureManager()
{
}

int32 FSlateD3DTextureManager::GetNumAtlasPages() const
{
	return PrecachedTextureAtlases.Num() + VectorGraphicsCache->GetNumAtlasPages();
}

FSlateShaderResource* FSlateD3DTextureManager::GetAtlasPageResource(const int32 InIndex) const
{
	return InIndex < PrecachedTextureAtlases.Num() ? PrecachedTextureAtlases[InIndex]->GetAtlasTexture() : VectorGraphicsCache->GetAtlasPageResource(InIndex - PrecachedTextureAtlases.Num());
}


bool FSlateD3DTextureManager::IsAtlasPageResourceAlphaOnly(const int32 InIndex) const
{
	return false;
}

#if WITH_ATLAS_DEBUGGING
FAtlasSlotInfo FSlateD3DTextureManager::GetAtlasSlotInfoAtPosition(FIntPoint InPosition, int32 AtlasIndex) const
{
	const FSlateTextureAtlas* Atlas = nullptr;

	bool bIsPrecachedTextureAtlases = PrecachedTextureAtlases.IsValidIndex(AtlasIndex);
	if (bIsPrecachedTextureAtlases)
	{
		Atlas = PrecachedTextureAtlases[AtlasIndex].Get();
	}
	else
	{
		Atlas = VectorGraphicsCache->GetAtlas(AtlasIndex - PrecachedTextureAtlases.Num());
	}

	FAtlasSlotInfo NewInfo;
	if(Atlas)
	{
		const FAtlasedTextureSlot* Slot = Atlas->GetSlotAtPosition(InPosition);
		if (Slot)
		{
			NewInfo.AtlasSlotRect = FSlateRect(FVector2f(Slot->X, Slot->Y), FVector2f(Slot->X + Slot->Width, Slot->Y + Slot->Height));
			NewInfo.TextureName = bIsPrecachedTextureAtlases ? AtlasDebugData.FindRef(Slot) : VectorGraphicsCache->GetAtlasDebugData(Slot);
		}
	}

	return NewInfo;
}
#endif

void FSlateD3DTextureManager::LoadUsedTextures()
{
	TArray< const FSlateBrush* > Resources;
	FSlateStyleRegistry::GetAllResources( Resources );

	CreateTextures( Resources );
}

void FSlateD3DTextureManager::LoadStyleResources( const ISlateStyle& Style )
{
	TArray< const FSlateBrush* > Resources;
	Style.GetResources( Resources );

	CreateTextures( Resources );
}


void FSlateD3DTextureManager::CreateTextures( const TArray< const FSlateBrush* >& Resources )
{
	TMap<FName,FNewTextureInfo> TextureInfoMap;
	
	for( int32 ResourceIndex = 0; ResourceIndex < Resources.Num(); ++ResourceIndex )
	{
		const FSlateBrush& Brush = *Resources[ResourceIndex];
		const FName TextureName = Brush.GetResourceName();

		if(Brush.GetImageType() != ESlateBrushImageType::Vector && TextureName != NAME_None && !ResourceMap.Contains(TextureName) )
		{
			// Find the texture or add it if it doesn't exist (only load the texture once)
			FNewTextureInfo& Info = TextureInfoMap.FindOrAdd( TextureName );

			Info.bSrgb = (Brush.ImageType != ESlateBrushImageType::Linear);

			// Only atlas the texture if none of the brushes that use it tile it
			Info.bShouldAtlas &= (Brush.Tiling == ESlateBrushTileType::NoTile && Info.bSrgb );


			if( !Info.TextureData.IsValid())
			{
				uint32 Width = 0;
				uint32 Height = 0;
				TArray<uint8> RawData;
				bool bSucceeded = LoadTexture( Brush, Width, Height, RawData );

				const uint32 Stride = 4; // RGBA

				Info.TextureData = MakeShareable( new FSlateTextureData( Width, Height, Stride, RawData ) );

				const bool bTooLargeForAtlas = (Width >= 256 || Height >= 256);

				Info.bShouldAtlas &= !bTooLargeForAtlas;

				if( !bSucceeded )
				{
					TextureInfoMap.Remove( TextureName );
				}
			}
		}
	}

	TextureInfoMap.ValueSort( FCompareFNewTextureInfoByTextureSize() );

	for( TMap<FName,FNewTextureInfo>::TConstIterator It(TextureInfoMap); It; ++It )
	{
		const FNewTextureInfo& Info = It.Value();
		FName TextureName = It.Key();
		FString NameStr = TextureName.ToString();

		FSlateShaderResourceProxy* NewTexture = GenerateTextureResource( Info, TextureName );

		ResourceMap.Add( TextureName, NewTexture );
	}
}

/**
 * Returns a texture with the passed in name or NULL if it cannot be found.
 */
FSlateShaderResourceProxy* FSlateD3DTextureManager::GetShaderResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale)
{
	FSlateShaderResourceProxy* Texture = NULL;
	if (Brush.GetImageType() == ESlateBrushImageType::Vector)
	{
		Texture = GetVectorResource(Brush, LocalSize, DrawScale);
	}
	else if( Brush.IsDynamicallyLoaded() )
	{
		Texture = GetDynamicTextureResource( Brush );
	}
	else
	{
		Texture = ResourceMap.FindRef( Brush.GetResourceName() );
	}

	return Texture;
}

ISlateAtlasProvider* FSlateD3DTextureManager::GetTextureAtlasProvider()
{
	return this;
}

bool FSlateD3DTextureManager::LoadTexture( const FSlateBrush& InBrush, uint32& OutWidth, uint32& OutHeight, TArray<uint8>& OutDecodedImage )
{
	FString ResourcePath = GetResourcePath( InBrush );

	uint32 BytesPerPixel = 4;
	bool bSucceeded = false;
	TArray<uint8> RawFileData;
	if( FFileHelper::LoadFileToArray( RawFileData, *ResourcePath, FILEREAD_Silent) )
	{
		IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>( FName("ImageWrapper") );

		//Try and determine format, if that fails assume PNG
		EImageFormat ImageFormat = ImageWrapperModule.DetectImageFormat(RawFileData.GetData(), RawFileData.Num());
		if (ImageFormat == EImageFormat::Invalid)
		{
			ImageFormat = EImageFormat::PNG;
		}
		TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(ImageFormat);

		if ( ImageWrapper.IsValid() && ImageWrapper->SetCompressed( RawFileData.GetData(), RawFileData.Num() ) )
		{
			OutWidth = ImageWrapper->GetWidth();
			OutHeight = ImageWrapper->GetHeight();

			if (ImageWrapper->GetRaw(ERGBFormat::BGRA, 8, OutDecodedImage))
			{
				bSucceeded = true;
			}
			else
			{
				UE_LOGF(LogSlateD3D, Warning, "Invalid texture format %d for Slate resource only RGBA and RGB pngs are supported: %ls", int(ImageFormat), *InBrush.GetResourceName().ToString() );
			}
		}
		else
		{
			UE_LOGF(LogSlateD3D, Warning, "Only pngs are supported in Slate. [%ls] '%ls'", *InBrush.GetResourceName().ToString(), *ResourcePath);
		}
	}
	else
	{
		UE_CLOGF(!IsRunningCommandlet(), LogSlateD3D, Warning, "Could not load file for Slate resource: [%ls] '%ls'", *InBrush.GetResourceName().ToString(), *ResourcePath);
	}

	return bSucceeded;
}

FSlateShaderResourceProxy* FSlateD3DTextureManager::CreateColorTexture( const FName TextureName, FColor InColor )
{
	FNewTextureInfo Info;

	TArray<uint8> RawData;
	RawData.AddUninitialized(4);
	RawData[0] = InColor.R;
	RawData[1] = InColor.G;
	RawData[2] = InColor.B;
	RawData[3] = InColor.A;
	Info.bShouldAtlas = false;
	
	uint32 Width = 1;
	uint32 Height = 1;
	uint32 Stride = 4;
	Info.TextureData = MakeShareable( new FSlateTextureData( Width, Height, Stride, RawData ) );


	FSlateShaderResourceProxy* NewTexture = GenerateTextureResource( Info, TextureName );
	checkSlow( !ResourceMap.Contains( TextureName ) );
	ResourceMap.Add( TextureName, NewTexture );

	return NewTexture;
}

FSlateShaderResourceProxy* FSlateD3DTextureManager::GetDynamicTextureResource( const FSlateBrush& InBrush )
{
	const FName ResourceName = InBrush.GetResourceName();

	// Bail out if we already have this texture loaded
	TSharedPtr<FDynamicTextureResource> TextureResource = DynamicTextureMap.FindRef(ResourceName);
	if (TextureResource.IsValid())
	{
		return TextureResource->Proxy;
	}

	if( InBrush.IsDynamicallyLoaded() )
	{		
		uint32 Width = 0;
		uint32 Height = 0;
		TArray<uint8> RawData;
		bool bSucceeded = LoadTexture( InBrush, Width, Height, RawData );

		if( bSucceeded )
		{
			return CreateDynamicTextureResource(ResourceName, Width, Height, RawData);
		}
		else
		{
			TextureResource = MakeShareable( new FDynamicTextureResource( NULL ) );

			// Add the null texture so we dont continuously try to load it.
			DynamicTextureMap.Add( ResourceName, TextureResource );
		}
	}

	if( TextureResource.IsValid() )
	{
		return TextureResource->Proxy;
	}

	// dynamic texture was not found or loaded
	return  NULL;
}

FSlateShaderResourceProxy* FSlateD3DTextureManager::GetVectorResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale)
{
	return VectorGraphicsCache->GetShaderResource(Brush, LocalSize, DrawScale);
}

FSlateShaderResourceProxy* FSlateD3DTextureManager::CreateDynamicTextureResource(FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& RawData)
{
	// Bail out if we already have this texture loaded
	TSharedPtr<FDynamicTextureResource> TextureResource = DynamicTextureMap.FindRef(ResourceName);
	if (TextureResource.IsValid())
	{
		return TextureResource->Proxy;
	}

	// Keep track of non-atlased textures so we can free their resources later
	FSlateD3DTexture* LoadedTexture = new FSlateD3DTexture(Width, Height);

	TextureResource = MakeShareable(new FDynamicTextureResource(LoadedTexture));

	D3D11_SUBRESOURCE_DATA InitData;
	InitData.pSysMem = RawData.GetData();
	InitData.SysMemPitch = Width * 4;

	FNewTextureInfo Info;
	Info.bShouldAtlas = false;
	LoadedTexture->Init(Info.bSrgb ? DXGI_FORMAT_B8G8R8A8_UNORM_SRGB : DXGI_FORMAT_B8G8R8A8_UNORM, &InitData);

	TextureResource->Proxy->ActualSize = FIntPoint(Width, Height);
	TextureResource->Proxy->Resource = TextureResource->D3DTexture;

	// Map the new resource for the UTexture so we don't have to load again
	DynamicTextureMap.Add(ResourceName, TextureResource);

	return TextureResource->Proxy;
}

void FSlateD3DTextureManager::ReleaseDynamicTextureResource( const FSlateBrush& InBrush )
{
	// Note: Only dynamically loaded or utexture brushes can be dynamically released
	if( InBrush.IsDynamicallyLoaded() )
	{
		const FName ResourceName = InBrush.GetResourceName();
		TSharedPtr<FDynamicTextureResource> TextureResource = DynamicTextureMap.FindRef(ResourceName);
		if( TextureResource.IsValid() )
		{
			//remove it from the texture map
			DynamicTextureMap.Remove( ResourceName );
			
			check( TextureResource.IsUnique() );
		}
	}
}

void FSlateD3DTextureManager::UpdateCache()
{
	for (TUniquePtr<FSlateTextureAtlasD3D>& Atlas : PrecachedTextureAtlases)
	{
		Atlas->ConditionalUpdateTexture();
	}

	VectorGraphicsCache->UpdateCache();
}

void FSlateD3DTextureManager::ConditionalFlushCache()
{
	VectorGraphicsCache->ConditionalFlushCache();
}

FSlateShaderResourceProxy* FSlateD3DTextureManager::GenerateTextureResource( const FNewTextureInfo& Info, FName TextureName )
{
	FSlateShaderResourceProxy* NewProxy = NULL;

	const uint32 Width = Info.TextureData->GetWidth();
	const uint32 Height = Info.TextureData->GetHeight();

	if( Info.bShouldAtlas )
	{
		const uint32 AtlasSize = 1024;
		// 4 bytes per pixel
		const uint32 AtlasStride = 4; 
		// always use one pixel padding.
		const uint8 Padding = 1;
		const FAtlasedTextureSlot* NewSlot = nullptr;

		FSlateTextureAtlasD3D* Atlas = nullptr;
		// Get the last atlas and find a slot for the texture
		for( int32 AtlasIndex = 0; AtlasIndex < PrecachedTextureAtlases.Num(); ++AtlasIndex )
		{
			Atlas = PrecachedTextureAtlases[AtlasIndex].Get();
			NewSlot = Atlas->AddTexture( Width, Height, Info.TextureData->GetRawBytes() );
			if( NewSlot )
			{
				break;
			}
		}

		// No new slot was found in any atlas so we have to make another one
		if( !NewSlot )
		{
			// A new slot in the atlas could not be found, make a new atlas and add the texture to it
			TUniquePtr<FSlateTextureAtlasD3D> NewAtlas = FSlateD3DTextureAtlasFactory::CreateTextureAtlasInternal(AtlasSize, AtlasStride, ESlateTextureAtlasPaddingStyle::DilateBorder, true);
			NewSlot = NewAtlas->AddTexture( Width, Height, Info.TextureData->GetRawBytes() );
		
			Atlas = NewAtlas.Get();

			PrecachedTextureAtlases.Emplace(MoveTemp(NewAtlas));
		}

		check(Atlas && NewSlot);

#if WITH_ATLAS_DEBUGGING
		AtlasDebugData.Add(NewSlot, TextureName);
#endif
		// Create a proxy representing this texture in the atlas
		NewProxy = new FSlateShaderResourceProxy;
		NewProxy->Resource = Atlas->GetAtlasTexture();
		// Compute the sub-uvs for the location of this texture in the atlas, accounting for padding
		NewProxy->StartUV = FVector2f( (float)(NewSlot->X+Padding) / Atlas->GetWidth(), (float)(NewSlot->Y+Padding) / Atlas->GetHeight() );
		NewProxy->SizeUV = FVector2f( (float)(NewSlot->Width-Padding*2) / Atlas->GetWidth(), (float)(NewSlot->Height-Padding*2) / Atlas->GetHeight() );
		NewProxy->ActualSize = FIntPoint( Width, Height );
	}
	else
	{
		// The texture is not atlased create a new texture proxy and just point it to the actual texture
		NewProxy = new FSlateShaderResourceProxy;

		// Keep track of non-atlased textures so we can free their resources later
		TUniquePtr<FSlateD3DTexture> Texture = MakeUnique<FSlateD3DTexture>( Width, Height );
	
		NewProxy->Resource = Texture.Get();
		NewProxy->StartUV = FVector2f(0.0f,0.0f);
		NewProxy->SizeUV = FVector2f(1.0f, 1.0f);
		NewProxy->ActualSize = FIntPoint( Width, Height );

		D3D11_SUBRESOURCE_DATA InitData;
		InitData.pSysMem = Info.TextureData->GetRawBytes().GetData();
		InitData.SysMemPitch = Width * 4;

		Texture->Init( Info.bSrgb ? DXGI_FORMAT_B8G8R8A8_UNORM_SRGB : DXGI_FORMAT_B8G8R8A8_UNORM, &InitData );

		NonAtlasedTextures.Add(MoveTemp(Texture));
	}

	return NewProxy;
}
