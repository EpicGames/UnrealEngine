// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Textures/TextureAtlas.h"
#include "Rendering/ShaderResourceManager.h"
#include "SlateD3DTextures.h"

class FSlateVectorGraphicsCache;
class ISlateStyle;

/**
 * Stores a mapping of texture names to their loaded d3d resource.  Resources are loaded from disk and created on demand when needed                   
 */
class FSlateD3DTextureManager : public ISlateAtlasProvider, public FSlateShaderResourceManager
{
public:
	FSlateD3DTextureManager();
	virtual ~FSlateD3DTextureManager();

	/** ISlateAtlasProvider */
	virtual int32 GetNumAtlasPages() const override;
	virtual FSlateShaderResource* GetAtlasPageResource(const int32 InIndex) const override;
	virtual bool IsAtlasPageResourceAlphaOnly(const int32 InIndex) const override;
#if WITH_ATLAS_DEBUGGING
	virtual FAtlasSlotInfo GetAtlasSlotInfoAtPosition(FIntPoint InPosition, int32 AtlasIndex) const override;
#endif

	/**
	 * Loads and creates rendering resources for all used textures.  
	 * In this implementation all textures must be known at startup time or they will not be found
	 */
	void LoadUsedTextures();

	void LoadStyleResources( const ISlateStyle& Style );


	/**
	 * Returns a texture with the passed in name or NULL if it cannot be found.
	 */
	virtual FSlateShaderResourceProxy* GetShaderResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale) override;

	virtual ISlateAtlasProvider* GetTextureAtlasProvider() override;

	/**
	 * Creates a 1x1 texture of the specified color
	 * 
	 * @param TextureName	The name of the texture
	 * @param InColor		The color for the texture
	 */
	virtual FSlateShaderResourceProxy* CreateColorTexture( const FName TextureName, FColor InColor );

	FSlateShaderResourceProxy* CreateDynamicTextureResource(FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& RawData);

	/** 
	 * Releases a dynamic texture resource
	 * 
	 * @param InBrush	The brush to with texture to release
	 */
	void ReleaseDynamicTextureResource( const FSlateBrush& InBrush );

	void UpdateCache();
	void ConditionalFlushCache();
private:
	/** 
	 * Gets a dynamic texture resource
	 * 
	 * @param InBrush	The brush to with texture to get
	 */
	FSlateShaderResourceProxy* GetDynamicTextureResource( const FSlateBrush& InBrush );

	/** 
	 * Gets a vector graphics resource (may generate it internally)
	 * 
	 * @param InBrush	The brush to with texture to get
	 * @param LocalSize	The unscaled local size of the final image
	 * @param DrawScale	Any scaling applied to the final image
	 */
	FSlateShaderResourceProxy* GetVectorResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale);

	/** 
	 * Creates textures from files on disk and atlases them if possible
	 *
	 * @param Resources	The brush resources to load images for
	 */
	void CreateTextures( const TArray< const FSlateBrush* >& Resources );

	bool LoadTexture( const FSlateBrush& InBrush, uint32& OutWidth, uint32& OutHeight, TArray<uint8>& OutDecodedImage );

	/**
	 * Generates rendering resources for a texture
	 * 
	 * @param Info	Information on how to generate the texture resource
	 */
	FSlateShaderResourceProxy* GenerateTextureResource( const FNewTextureInfo& Info, FName TextureName );
	
private:

	/** Represents a dynamic resource for rendering */
	struct FDynamicTextureResource
	{
		FSlateShaderResourceProxy* Proxy;
		FSlateD3DTexture* D3DTexture;

		FDynamicTextureResource( FSlateD3DTexture* ExistingTexture );
		~FDynamicTextureResource();
	};

	/** Map of all active dynamic texture objects being used for brush resources */
	TMap<FName, TSharedPtr<FDynamicTextureResource> > DynamicTextureMap;

	/** Static texture atlases */
	TArray<TUniquePtr<FSlateTextureAtlasD3D>> PrecachedTextureAtlases;
	
	/** Static non atlased textures */
	TArray<TUniquePtr<FSlateD3DTexture>> NonAtlasedTextures;

	/** Cache for vector graphic atlases */
	TUniquePtr<FSlateVectorGraphicsCache> VectorGraphicsCache;
};
