// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/D3D/SlateD3DTextures.h"
#include "Windows/D3D/SlateD3DRenderer.h"

void FSlateD3DTexture::Init( DXGI_FORMAT InFormat, D3D11_SUBRESOURCE_DATA* InitalData, bool bUpdatable, bool bUseStagingTexture)
{
	check(bUseStagingTexture ? bUpdatable : true); // It only makes sense to use a staging texture if updatable is true
	D3D11_USAGE Usage = D3D11_USAGE_DEFAULT;
	uint32 InCPUAccessFlags = 0;
	uint32 BindFlags = D3D11_BIND_SHADER_RESOURCE;

	// Todo: This should work for the formats we are using, but we should probably do this more properly
	BytesPerPixel = InFormat == DXGI_FORMAT_A8_UNORM ? 1 : 4;

	// Create the texture resource
	D3D11_TEXTURE2D_DESC TexDesc;
	TexDesc.Width = SizeX;
	TexDesc.Height = SizeY;
	TexDesc.MipLevels = 1;
	TexDesc.ArraySize = 1;
	TexDesc.Format = InFormat;
	TexDesc.SampleDesc.Count = 1;
	TexDesc.SampleDesc.Quality = 0;
	TexDesc.Usage = (bUpdatable && !bUseStagingTexture) ? D3D11_USAGE_DYNAMIC : D3D11_USAGE_DEFAULT;
	TexDesc.BindFlags = D3D11_BIND_SHADER_RESOURCE;
	TexDesc.CPUAccessFlags = (bUpdatable && !bUseStagingTexture) ? D3D11_CPU_ACCESS_WRITE : 0;
	TexDesc.MiscFlags = 0;

	HRESULT Hr = GD3DDevice->CreateTexture2D( &TexDesc, InitalData, D3DTexture.GetInitReference() );
	if (SUCCEEDED(Hr))
	{
		// Create the shader accessable view of the texture
		D3D11_SHADER_RESOURCE_VIEW_DESC SrvDesc;
		SrvDesc.Format = TexDesc.Format;
		SrvDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2D;
		SrvDesc.Texture2D.MostDetailedMip = 0;
		SrvDesc.Texture2D.MipLevels = 1;

		// Create the shader resource view.
		Hr = GD3DDevice->CreateShaderResourceView(D3DTexture, &SrvDesc, ShaderResource.GetInitReference());
		if (SUCCEEDED(Hr))
		{
			// Crate a staging texture for updating
			if (bUpdatable && bUseStagingTexture)
			{
				TexDesc.Usage = D3D11_USAGE_STAGING;
				TexDesc.BindFlags = 0;
				TexDesc.CPUAccessFlags = D3D11_CPU_ACCESS_WRITE | D3D11_CPU_ACCESS_READ;

				Hr = GD3DDevice->CreateTexture2D(&TexDesc, InitalData, StagingTexture.GetInitReference());

				if (FAILED(Hr))
				{
					LogSlateD3DRendererFailure(TEXT("FSlateD3DTexture::Init() - ID3D11Device::CreateTexture2D"), Hr);
					GEncounteredCriticalD3DDeviceError = true;
					StagingTexture = nullptr;
				}
			}
			else
			{
				StagingTexture = nullptr;
			}
		}
		else
		{
			LogSlateD3DRendererFailure(TEXT("FSlateD3DTexture::Init() - ID3D11Device::CreateShaderResourceView"), Hr);
			GEncounteredCriticalD3DDeviceError = true;
		}
	}
	else
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DTexture::Init() - ID3D11Device::CreateTexture2D"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
	}
}

void FSlateD3DTexture::ResizeTexture(uint32 Width, uint32 Height)
{
	// Seems only way to resize d3d texture is recreate it
	SizeX = Width;
	SizeY = Height;
	D3D11_TEXTURE2D_DESC TextureDesc;
	D3DTexture->GetDesc(&TextureDesc);
	Init(TextureDesc.Format, NULL, true, StagingTexture.IsValid());
}

void FSlateD3DTexture::UpdateTextureRaw(const void* Buffer, const FIntRect& Dirty)
{
	bool bUseStagingTexture = StagingTexture.IsValid();
	ID3D11Texture2D* TextureToUpdate = bUseStagingTexture ? StagingTexture : D3DTexture;

	D3D11_BOX Region;
	Region.front = 0;
	Region.back = 1;

	if (bUseStagingTexture && Dirty.Area() > 0)
	{
		Region.left = Dirty.Min.X;
		Region.top = Dirty.Min.Y;
		Region.right = Dirty.Max.X;
		Region.bottom = Dirty.Max.Y;
	}
	else
	{
		Region.left = 0;
		Region.right = SizeX;
		Region.top = 0;
		Region.bottom = SizeY;
	}

	D3D11_MAPPED_SUBRESOURCE Resource;
	HRESULT Hr = GD3DDeviceContext->Map(TextureToUpdate, 0, bUseStagingTexture ? D3D11_MAP_READ_WRITE : D3D11_MAP_WRITE_DISCARD, 0, &Resource);

	if (SUCCEEDED(Hr))
	{
		uint32 SourcePitch = SizeX * BytesPerPixel;
		uint32 CopyRowBytes = (Region.right - Region.left) * BytesPerPixel;
		uint8* Destination = (uint8*)Resource.pData + Region.left * BytesPerPixel + Region.top * Resource.RowPitch;
		uint8* Source = (uint8*)Buffer + Region.left * BytesPerPixel + Region.top * SourcePitch;
		for (uint32 Row = Region.top; Row < Region.bottom; ++Row, Destination += Resource.RowPitch, Source += SourcePitch)

		{
			FMemory::Memcpy(Destination, Source, CopyRowBytes);
		}

		GD3DDeviceContext->Unmap(TextureToUpdate, 0);

		if (bUseStagingTexture)
		{
			GD3DDeviceContext->CopySubresourceRegion(D3DTexture, 0, Region.left, Region.top, Region.front, StagingTexture, 0, &Region);
		}
	}
	else
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DTexture::UpdateTextureRaw() - ID3D11DeviceContext::Map"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
	}
}

void FSlateD3DTexture::UpdateTexture(const TArray<uint8>& Bytes)
{
	UpdateTextureRaw(Bytes.GetData(), FIntRect());
}

void FSlateD3DTexture::UpdateTextureThreadSafeRaw(uint32 Width, uint32 Height, const void* Buffer, const FIntRect& Dirty)
{
	if (Width == SizeX && Height == SizeY)
	{
		UpdateTextureRaw(Buffer, Dirty);
	}
	else
	{
		ResizeTexture(Width, Height);
		UpdateTextureRaw(Buffer, FIntRect());
	}
}

void FSlateD3DTexture::UpdateTextureThreadSafeWithTextureData(FSlateTextureData* TextureData)
{
	UpdateTextureThreadSafeRaw(TextureData->GetWidth(), TextureData->GetHeight(), TextureData->GetRawBytesPtr());
	delete TextureData; 
}

void FSlateD3DTexture::Init(void* ShareHandleIn)
{
	ID3D11Device1* dev1 = nullptr;
	HRESULT Hr = GD3DDevice->QueryInterface(__uuidof(ID3D11Device1), reinterpret_cast<void**>(&dev1));
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DSharedTexture::Init() - ID3D11Device::QueryInterface"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	ID3D11Texture2D* tex = nullptr;
	Hr = dev1->OpenSharedResource1(ShareHandleIn, __uuidof(ID3D11Texture2D), (void**)(&tex));
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DSharedTexture::Init() - ID3D11Device::OpenSharedResource"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	D3D11_TEXTURE2D_DESC TexDesc;
	tex->GetDesc(&TexDesc);

	SizeX = TexDesc.Width;
	SizeY = TexDesc.Height;
	ShareHandle = ShareHandleIn;

	Init(TexDesc.Format, NULL, true, false);
}

void FSlateD3DTexture::UpdateTextureThreadSafeWithKeyedTextureHandle(void* TextureHandle, int KeyLockVal, int KeyUnlockVal, const FIntRect& InDirty)
{
	FIntRect Dirty = InDirty;
	if (ShareHandle != TextureHandle)
	{
		Init(TextureHandle);
		Dirty = FIntRect();
	}

	ID3D11Device1* dev1 = nullptr;
	HRESULT Hr = GD3DDevice->QueryInterface(__uuidof(ID3D11Device1), reinterpret_cast<void**>(&dev1));
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DTexture::UpdateTextureThreadSafeWithKeyedTextureHandle() - ID3D11Device::QueryInterface"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	ID3D11Texture2D* Tex = nullptr;
	Hr = dev1->OpenSharedResource1(ShareHandle, __uuidof(ID3D11Texture2D), (void**)(&Tex));
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DTexture::UpdateTextureThreadSafeWithKeyedTextureHandle() - ID3D11Device::OpenSharedResource"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	D3D11_TEXTURE2D_DESC TexDesc;
	Tex->GetDesc(&TexDesc);
	if (SizeX != TexDesc.Width || SizeY != TexDesc.Height)
	{
		ResizeTexture(TexDesc.Width, TexDesc.Height);
		Dirty = FIntRect();
	}

	TRefCountPtr<IDXGIKeyedMutex> KeyedMutex;
	Hr = Tex->QueryInterface(_uuidof(IDXGIKeyedMutex), (void**)&KeyedMutex);
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DTexture::UpdateTextureThreadSafeWithKeyedTextureHandle() - ID3D11Texture2D::IDXGIKeyedMutex"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}

	if (KeyedMutex)
	{
		if (KeyedMutex->AcquireSync(KeyLockVal, INFINITE) == S_OK)
		{
			D3D11_BOX Region;
			Region.front = 0;
			Region.back = 1;
			if (Dirty.Area() > 0)
			{
				Region.left = Dirty.Min.X;
				Region.top = Dirty.Min.Y;
				Region.right = Dirty.Max.X;
				Region.bottom = Dirty.Max.Y;
			}
			else
			{
				Region.left = 0;
				Region.right = SizeX;
				Region.top = 0;
				Region.bottom = SizeY;
			}

			GD3DDeviceContext->CopySubresourceRegion(D3DTexture, 0, Region.left, Region.top, Region.front, Tex, 0, &Region);

			KeyedMutex->ReleaseSync(KeyUnlockVal);
		}
	}
	else
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DSharedTexture::UpdateTextureThreadSafeWithKeyedTextureHandle() - missing KeyedMutex"), E_FAIL);
		GEncounteredCriticalD3DDeviceError = true;
		return;
	}
}

FSlateTextureAtlasD3D::FSlateTextureAtlasD3D( uint32 Width, uint32 Height, uint32 StrideBytes, ESlateTextureAtlasPaddingStyle PaddingStyle )
	: FSlateTextureAtlas(Width, Height, StrideBytes, PaddingStyle, true)
	, AtlasTexture(nullptr)
{
	InitAtlasTexture();
}

FSlateTextureAtlasD3D::~FSlateTextureAtlasD3D()
{
	if( AtlasTexture )
	{
		delete AtlasTexture;
	}
}

void FSlateTextureAtlasD3D::InitAtlasTexture()
{
	check(!AtlasTexture);

	AtlasTexture = new FSlateD3DTexture(AtlasWidth, AtlasHeight);

	AtlasTexture->Init( DXGI_FORMAT_B8G8R8A8_UNORM_SRGB, nullptr, true );
}

void FSlateTextureAtlasD3D::ConditionalUpdateTexture()
{
	if (bNeedsUpdate)
	{
		AtlasTexture->UpdateTexture(AtlasData);
		bNeedsUpdate = false;
	}
}

FSlateFontAtlasD3D::FSlateFontAtlasD3D(uint32 Width, uint32 Height, ESlateFontAtlasContentType InContentType)
	: FSlateFontAtlas(Width, Height, InContentType, NoPadding)
{
	FontTexture = new FSlateD3DTexture(Width, Height);
	DXGI_FORMAT Format = DXGI_FORMAT_UNKNOWN;
	switch (InContentType)
	{
		case ESlateFontAtlasContentType::Alpha:
			Format = DXGI_FORMAT_A8_UNORM;
			break;
		case ESlateFontAtlasContentType::Color:
			Format = DXGI_FORMAT_B8G8R8A8_UNORM_SRGB;
			break;
		case ESlateFontAtlasContentType::Msdf:
			Format = DXGI_FORMAT_B8G8R8A8_UNORM;
			break;
		default:
			checkNoEntry();
			// Default to Color
			Format = DXGI_FORMAT_B8G8R8A8_UNORM_SRGB;
			break;
	}
	check(Format != DXGI_FORMAT_UNKNOWN);
	FontTexture->Init(Format, NULL, true, false);
}

FSlateFontAtlasD3D::~FSlateFontAtlasD3D()
{
	delete FontTexture;
}

void FSlateFontAtlasD3D::ConditionalUpdateTexture()
{
	if( bNeedsUpdate )
	{
		FontTexture->UpdateTexture(AtlasData);
		bNeedsUpdate = false;
	}
}
