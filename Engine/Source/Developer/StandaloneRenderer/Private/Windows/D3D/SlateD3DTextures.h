// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Textures/SlateShaderResource.h"
#include "Textures/SlateUpdatableTexture.h"
#include "Textures/TextureAtlas.h"
#include "Fonts/FontTypes.h"
#include "Templates/RefCounting.h"
#include "StandaloneRendererPlatformHeaders.h"

/**
 * Encapsulates a D3D11 texture that can be accessed by a shader                   
 */
class FSlateD3DTexture : public TSlateTexture< TRefCountPtr<ID3D11ShaderResourceView> >, public FSlateUpdatableTexture
{
public:
	FSlateD3DTexture( uint32 InSizeX, uint32 InSizeY )
		: TSlateTexture()
		, SizeX(InSizeX)
		, SizeY(InSizeY)
	{

	}

	FSlateD3DTexture()
		: TSlateTexture()
		, SizeX(0)
		, SizeY(0)
	{

	}

	
	~FSlateD3DTexture()
	{

	}

	void Init( DXGI_FORMAT InFormat, D3D11_SUBRESOURCE_DATA* InitalData = NULL, bool bUpdatable = false, bool bUseStagingTexture = false );
	void Init(void* ShareHandle); // shared handle variant 

	virtual void Cleanup() override { delete this; }

	uint32 GetWidth() const { return SizeX; }
	uint32 GetHeight() const { return SizeY; }

	TRefCountPtr<ID3D11Texture2D> GetTextureResource() const { return D3DTexture; }

	// FSlateUpdatableTexture interface
	virtual FSlateShaderResource* GetSlateResource() override {return this;}
	virtual void ResizeTexture( uint32 Width, uint32 Height ) override;
	virtual void UpdateTexture(const TArray<uint8>& Bytes) override;
	virtual void UpdateTextureThreadSafe(const TArray<uint8>& Bytes) override { UpdateTexture(Bytes); }
	virtual void UpdateTextureThreadSafeRaw(uint32 Width, uint32 Height, const void* Buffer, const FIntRect& Dirty = FIntRect()) override;
	virtual void UpdateTextureThreadSafeWithTextureData(FSlateTextureData* TextureData) override;
	virtual void UpdateTextureThreadSafeWithKeyedTextureHandle(void* TextureHandle, int KeyLockVal, int KeyUnlockVal, const FIntRect& Dirty = FIntRect()) override;
private:
	// Helper method used by the different UpdateTexture* methods
	void UpdateTextureRaw(const void* Buffer, const FIntRect& Dirty);
	/** Actual texture.  In D3D the SRV is used by the shader so our parent class holds that */
	TRefCountPtr<ID3D11Texture2D> D3DTexture;
	/** Texture for doing staged updating of the texture contents. */
	TRefCountPtr<ID3D11Texture2D> StagingTexture;
	uint32 SizeX;
	uint32 SizeY;
	uint32 BytesPerPixel;

	void* ShareHandle;
};

class FSlateTextureAtlasD3D : public FSlateTextureAtlas
{
public:
	FSlateTextureAtlasD3D( uint32 Width, uint32 Height, uint32 StrideBytes, ESlateTextureAtlasPaddingStyle PaddingStyle );
	~FSlateTextureAtlasD3D();


	virtual void ConditionalUpdateTexture() override;
	virtual void ReleaseResources() override {}

	FSlateD3DTexture* GetAtlasTexture() const { return AtlasTexture; }
private:
	void InitAtlasTexture();
private:
	FSlateD3DTexture* AtlasTexture;
};

/** 
 * Representation of a texture for fonts in which characters are packed tightly based on their bounding rectangle 
 */
class FSlateFontAtlasD3D : public FSlateFontAtlas
{
public:
	FSlateFontAtlasD3D(uint32 Width, uint32 Height, ESlateFontAtlasContentType InContentType);
	~FSlateFontAtlasD3D();

	/** FSlateFontAtlas interface */
	virtual void ConditionalUpdateTexture();
	virtual class FSlateShaderResource* GetSlateTexture() const override { return FontTexture; }
	virtual class FTextureResource* GetEngineTexture() override { return nullptr; }
	virtual void ReleaseResources() override {}

private:
	/** Texture used for rendering */
	FSlateD3DTexture* FontTexture;
};