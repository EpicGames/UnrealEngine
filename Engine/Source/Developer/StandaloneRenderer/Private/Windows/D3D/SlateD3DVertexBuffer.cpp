// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/D3D/SlateD3DVertexBuffer.h"
#include "Windows/D3D/SlateD3DRenderer.h"
#include "StandaloneRendererPrivate.h"

FSlateD3DVertexBuffer::FSlateD3DVertexBuffer()
: BufferSize(0), Stride(0)
{

}

FSlateD3DVertexBuffer::~FSlateD3DVertexBuffer()
{

}

/** Initializes the vertex buffers RHI resource. */
void FSlateD3DVertexBuffer::CreateBuffer( uint32 InStride )
{
	Stride = InStride;

	if( BufferSize == 0 )
	{
		// @todo: better default size.  Probably should be based on how much we know will always bee needed by the editor
		BufferSize = 2000 * Stride;
	}

	D3D11_BUFFER_DESC BufferDesc;

	BufferDesc.Usage = D3D11_USAGE_DYNAMIC;
	BufferDesc.ByteWidth = BufferSize;
	BufferDesc.BindFlags = D3D11_BIND_VERTEX_BUFFER;
	BufferDesc.CPUAccessFlags = D3D11_CPU_ACCESS_WRITE ;
	BufferDesc.MiscFlags = 0;

	HRESULT Hr = GD3DDevice->CreateBuffer( &BufferDesc, NULL, Buffer.GetInitReference() );
	if (FAILED(Hr))
	{
		LogSlateD3DRendererFailure(TEXT("FSlateD3DVertexBuffer::CreateBuffer() - ID3D11Device::CreateBuffer"), Hr);
		GEncounteredCriticalD3DDeviceError = true;
	}
}

/** Releases the vertex buffers RHI resource. */
void FSlateD3DVertexBuffer::DestroyBuffer()
{
	Buffer.SafeRelease();
}

/** Resizes the buffer to the passed in size.  Preserves internal data*/
void FSlateD3DVertexBuffer::ResizeBuffer( uint32 NewSize )
{
	// Buffer should be created first
	check( BufferSize > 0 );

	if( NewSize > BufferSize )
	{
		uint8 *SavedVertices = NULL;

		{
			void *Vertices = Lock(0);

			SavedVertices = new uint8[BufferSize];
			FMemory::Memcpy( SavedVertices, Vertices, BufferSize );

			Unlock();
		}

		D3D11_BUFFER_DESC BufferDesc;
		BufferDesc.Usage = D3D11_USAGE_DYNAMIC;
		BufferDesc.ByteWidth = NewSize;
		BufferDesc.BindFlags = D3D11_BIND_VERTEX_BUFFER;
		BufferDesc.CPUAccessFlags = D3D11_CPU_ACCESS_WRITE ;
		BufferDesc.MiscFlags = 0;

		HRESULT Hr = GD3DDevice->CreateBuffer( &BufferDesc, NULL, Buffer.GetInitReference() );
		if (SUCCEEDED(Hr))
		{
			if (SavedVertices)
			{
				void *Vertices = Lock(0);

				FMemory::Memcpy(Vertices, SavedVertices, BufferSize);

				Unlock();
			}

			BufferSize = NewSize;
		}
		else
		{
			LogSlateD3DRendererFailure(TEXT("FSlateD3DVertexBuffer::ResizeBuffer() - ID3D11Device::CreateBuffer"), Hr);
			GEncounteredCriticalD3DDeviceError = true;
		}

		delete[] SavedVertices;
	}
}

/** Locks the index buffer, returning a pointer to the indices */
void* FSlateD3DVertexBuffer::Lock( uint32 Offset )
{
	D3D11_MAPPED_SUBRESOURCE Resource;
	GD3DDeviceContext->Map( Buffer, 0, D3D11_MAP_WRITE_DISCARD, 0, &Resource );
	void* Data = Resource.pData;

	return (void*)((uint8*)Data + Offset);
}

void FSlateD3DVertexBuffer::Unlock()
{
	GD3DDeviceContext->Unmap( Buffer, 0 );
}
