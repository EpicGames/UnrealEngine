// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include "StandaloneRendererPlatformHeaders.h"

/** 
 * Vertex buffer containing all Slate vertices
 */
class FSlateD3DVertexBuffer
{
public:
	FSlateD3DVertexBuffer();
	~FSlateD3DVertexBuffer();

	/** Initializes the vertex buffers RHI resource. */
	void CreateBuffer( uint32 InStride );

	/** Releases the vertex buffers RHI resource. */
	void DestroyBuffer();

	/** Returns the size of the buffer in bytes. */
	uint32 GetBufferSize() const { return BufferSize; }

	/** Resizes the buffer to the passed in size.  Preserves internal data*/
	void ResizeBuffer( uint32 NewSize );

	void* Lock( uint32 Offset );
	void Unlock();

	TRefCountPtr<ID3D11Buffer> GetResource() { return Buffer; }
private:
	/** The size of the buffer in bytes. */
	uint32 BufferSize;
	uint32 Stride;
	TRefCountPtr<ID3D11Buffer> Buffer;
	/** Hidden copy methods. */
	FSlateD3DVertexBuffer( const FSlateD3DVertexBuffer& );
	void operator=(const FSlateD3DVertexBuffer& );
};
