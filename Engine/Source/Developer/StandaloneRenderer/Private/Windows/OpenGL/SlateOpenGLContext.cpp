// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandaloneRendererPrivate.h"

#include "HAL/Platform.h"
#include "OpenGL/SlateOpenGLExtensions.h"
#include "OpenGL/SlateOpenGLRenderer.h"

/**
 * A dummy wndproc.
 */
static LRESULT CALLBACK DummyGLWndproc(HWND hWnd, uint32 Message, WPARAM wParam, LPARAM lParam)
{
	return DefWindowProc(hWnd, Message, wParam, lParam);
}

/**
 * Create a dummy window used to construct OpenGL contexts.
 */
static HWND CreateDummyGLWindow()
{
	const TCHAR* WindowClassName = TEXT("DummyGLWindow");

	// Register a dummy window class.
	static bool bInitializedWindowClass = false;
	if (!bInitializedWindowClass)
	{
		WNDCLASS wc;

		bInitializedWindowClass = true;
		FMemory::Memzero(wc);
		wc.style = CS_OWNDC;
		wc.lpfnWndProc = DummyGLWndproc;
		wc.cbClsExtra = 0;
		wc.cbWndExtra = 0;
		wc.hInstance = NULL;
		wc.hIcon = NULL;
		wc.hCursor = NULL;
		wc.hbrBackground = (HBRUSH)(COLOR_MENUTEXT);
		wc.lpszMenuName = NULL;
		wc.lpszClassName = WindowClassName;
		ATOM ClassAtom = ::RegisterClass(&wc);
		check(ClassAtom);
	}

	// Create a dummy window.
	HWND WindowHandle = CreateWindowEx(
		WS_EX_WINDOWEDGE,
		WindowClassName,
		NULL,
		WS_POPUP,
		0, 0, 1, 1,
		NULL, NULL, NULL, NULL);
	check(WindowHandle);

	return WindowHandle;
}

FSlateOpenGLContext::FSlateOpenGLContext()
:	WindowHandle(NULL)
,	WindowDC(NULL)
,	Context(NULL)
,	bReleaseWindowOnDestroy(false)
{
}

FSlateOpenGLContext::~FSlateOpenGLContext()
{
	Destroy();
}

void FSlateOpenGLContext::Initialize( void* InWindow, const FSlateOpenGLContext* SharedContext )
{
	WindowHandle = (HWND)InWindow;

	if( WindowHandle == NULL )
	{
		WindowHandle = CreateDummyGLWindow();
		bReleaseWindowOnDestroy = true;
	}

	// Create a pixel format descriptor for this window
	PIXELFORMATDESCRIPTOR PFD;
	FMemory::Memzero( &PFD, sizeof(PIXELFORMATDESCRIPTOR) );

	PFD.nSize = sizeof(PIXELFORMATDESCRIPTOR);
	PFD.nVersion = 1;
	PFD.dwFlags = PFD_DRAW_TO_WINDOW | PFD_SUPPORT_OPENGL| PFD_DOUBLEBUFFER;
	PFD.iPixelType = PFD_TYPE_RGBA;
	PFD.cColorBits = 32; 
	PFD.cStencilBits = 0;
	PFD.cDepthBits = 0;
	PFD.iLayerType = PFD_MAIN_PLANE;

	WindowDC = GetDC( WindowHandle );
	check(WindowDC);

	// Get the pixel format for this window based on our descriptor
	uint32 PixelFormat = ChoosePixelFormat( WindowDC, &PFD );
	// Ensure a pixel format could be found
	check(PixelFormat);

	BOOL Result = SetPixelFormat( WindowDC, PixelFormat, &PFD );
	// Ensure the pixel format could be set on this window
	check(Result);

	Context = wglCreateContext( WindowDC );

	// Make the new window active for drawing.  We can't call any OpenGL functions without an active rendering context
	Result = wglMakeCurrent( WindowDC, Context );
	check( Result );

PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	wglCreateContextAttribsARB = ( PFNWGLCREATECONTEXTATTRIBSARBPROC )wglGetProcAddress( "wglCreateContextAttribsARB" );
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	check( wglCreateContextAttribsARB );

	int32 ContextAttribs[] =
	{
		WGL_CONTEXT_MAJOR_VERSION_ARB, 3,
		WGL_CONTEXT_MINOR_VERSION_ARB, 2,
		WGL_CONTEXT_PROFILE_MASK_ARB, WGL_CONTEXT_COMPATIBILITY_PROFILE_BIT_ARB,
		WGL_CONTEXT_FLAGS_ARB, WGL_CONTEXT_FORWARD_COMPATIBLE_BIT_ARB,
		0
	};

	HGLRC OldContext = Context;
	Context = wglCreateContextAttribsARB( WindowDC, SharedContext ? SharedContext->Context : NULL, ContextAttribs );
	wglMakeCurrent( NULL, NULL );
	wglDeleteContext( OldContext );
	wglMakeCurrent( WindowDC, Context );
}

void FSlateOpenGLContext::Destroy()
{
	if( WindowHandle != NULL )
	{
		wglMakeCurrent( NULL, NULL );
		wglDeleteContext( Context );
		ReleaseDC( WindowHandle, WindowDC );
		WindowDC = NULL;
		Context = NULL;

		if( bReleaseWindowOnDestroy )
		{
			DestroyWindow( WindowHandle );
		}
		WindowHandle = NULL;
	}
}

void FSlateOpenGLContext::MakeCurrent()
{
	wglMakeCurrent( WindowDC, Context );
}
