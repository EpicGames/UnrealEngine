// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#import <UIKit/UIKit.h>
#import <QuartzCore/QuartzCore.h>
#import <GLKit/GLKit.h>

@interface SlateOpenGLESView : GLKView

@property(retain) EAGLContext* Context;

@end

@interface SlateOpenGLESViewController : UIViewController

@end
