// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FSlateRenderer;

/**
 * Single function to create the standalone renderer for the running platform
 */
STANDALONERENDERER_API TSharedRef<FSlateRenderer> GetStandardStandaloneRenderer();

// @todo: Add GetModuleRenderer(const TCHAR* ModuleName) that can get a renderer from a module - this isn't needed any time soon,
// but that's the idea here
