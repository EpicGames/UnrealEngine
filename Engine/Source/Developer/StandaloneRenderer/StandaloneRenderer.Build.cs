// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StandaloneRenderer : ModuleRules
{
	public StandaloneRenderer(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"ImageWrapper",
				"InputCore",
				"SlateCore",
				"SlateNullRenderer",
			}
			);

		AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenGL");

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			// @todo: This should be private? Not sure!!
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11");
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicFrameworks.AddRange(new string[] { "QuartzCore", "IOSurface" });
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "SDL3");
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS))
		{
			PublicFrameworks.AddRange(new string[] { "OpenGLES", "GLKit" });
			// weak for IOS8 support since CAMetalLayer is in QuartzCore
			PublicWeakFrameworks.AddRange(new string[] { "QuartzCore" });
		}

		RuntimeDependencies.Add("$(EngineDir)/Shaders/StandaloneRenderer/...", StagedFileType.UFS);
	}
}
